{
	"FileVersion" : 3,
	"Version" : 1,
	"VersionName" : "1.0",
	"FriendlyName" : "Media Player Editor",
	"Description" : "Content Editor for MediaPlayer Assets.",
	"Category" : "Editor",
	"CreatedBy" : "Epic Games, Inc.",
	"CreatedByURL" : "https://epicgames.com",
  "DocsURL" : "https://forums.unrealengine.com/showthread.php?46879-Media-Framework-Documentation-for-4-5-Preview",
	"MarketplaceURL" : "",
	"SupportURL" : "",
	"EnabledByDefault" : true,
	"CanContainContent" : false,
	"IsBetaVersion" : false,
	"Installed" : false,
	"Modules" :
	[
		{
			"Name" : "MediaPlayerEditor",
			"Type" : "Editor",
			"LoadingPhase" : "PostEngineInit"
		}
	]
}
