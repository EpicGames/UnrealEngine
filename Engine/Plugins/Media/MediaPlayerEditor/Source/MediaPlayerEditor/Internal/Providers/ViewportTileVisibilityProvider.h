// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Assets/MediaTileVisibility.h"
#include "HAL/CriticalSection.h"
#include "Math/IntPoint.h"

#define UE_API MEDIAPLAYEREDITOR_API

struct FMediaSequenceInfo;

namespace UE::MediaPlayerEditor
{

/**
 * Tile-visibility provider for widgets that show a media texture in a viewport (no zoom/pan/sub-region).
 * Picks the integer mip pair bracketing log2(SourceSize / DisplaySize) and marks every tile at
 * those mips visible. Both mips are needed because trilinear filtering blends them - loading
 * just one starves the blend.
 *
 * Threading: SetDisplaySizePx is game-thread; GatherVisibleTiles runs on a resolver worker
 * thread and may be called concurrently with itself. Mutex protects DisplaySizePx.
 *
 * Lifetime: consumer holds the TSharedPtr; resolver tracks it weakly. Drop the TSharedPtr to detach.
 */
class FViewportTileVisibilityProvider : public FMediaTileVisibilityProvider
{
public:
	/** Update the on-screen pixel size of the widget hosting this provider. Game-thread only. */
	UE_API void SetDisplaySizePx(FIntPoint InSizePx);

	//~ Begin IMediaTileVisibilityProvider
	UE_API virtual void GatherVisibleTiles(const FMediaSequenceInfo& InSequenceInfo,
		FMediaTileVisibilityRequest& OutRequest) const override;

	UE_API virtual bool IsAlive() const override;
	//~ End IMediaTileVisibilityProvider

private:
	/** Guards DisplaySizePx against the cross-thread access between SetDisplaySizePx and GatherVisibleTiles. */
	mutable FCriticalSection Mutex;

	/** Latest on-screen pixel size pushed by the consumer. Zero until the first SetDisplaySizePx call. */
	FIntPoint DisplaySizePx = FIntPoint::ZeroValue;
};

} // namespace UE::MediaPlayerEditor

#undef UE_API
