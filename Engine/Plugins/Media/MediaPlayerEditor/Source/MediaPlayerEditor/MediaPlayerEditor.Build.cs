// Copyright Epic Games, Inc. All Rights Reserved.

using System.IO;
using UnrealBuildTool;

public class MediaPlayerEditor : ModuleRules
{
	public MediaPlayerEditor(ReadOnlyTargetRules Target) : base(Target)
	{
		DynamicallyLoadedModuleNames.AddRange(
			new string[] {
				"AssetTools",
				"MainFrame",
				"Media",
				"WorkspaceMenuStructure",
			});

		PublicDependencyModuleNames.AddRange(
			new string[] {
				"AssetDefinition",
				"Core",
				"Slate",
				"SlateCore"
			});

		PrivateDependencyModuleNames.AddRange(
			new string[] {
				"AudioMixer",
				"ComponentVisualizers",
				"ContentBrowser",
				"CoreUObject",
				"ApplicationCore",
				"DesktopPlatform",
				"DesktopWidgets",	
				"EditorWidgets",
				"Engine",
				"InputCore",
				"MediaAssets",
				"MediaUtils",
				"PropertyEditor",
				"RenderCore",
				"Renderer",
				"RHI",
				"TextureEditor",
				"EditorFramework",
				"UnrealEd",
				"ToolMenus",
			});

		PrivateIncludePathModuleNames.AddRange(
			new string[] {
				"AssetTools",
				"Media",
			});
	}
}
