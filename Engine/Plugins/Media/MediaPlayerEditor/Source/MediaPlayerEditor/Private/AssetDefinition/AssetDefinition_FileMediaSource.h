// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "FileMediaSource.h"
#include "AssetDefinition/AssetDefinition_MediaSource.h"
#include "AssetDefinition_FileMediaSource.generated.h"

UCLASS()
class UAssetDefinition_FileMediaSource : public UAssetDefinition_MediaSource
{
	GENERATED_BODY()
public:
	// UAssetDefinition Begin
	virtual FText GetAssetDisplayName() const override { return NSLOCTEXT("AssetDefinition", "AssetDefinition_FileMediaSource", "File Media Source"); }
	virtual TSoftClassPtr<UObject> GetAssetClass() const override { return UFileMediaSource::StaticClass(); }
	virtual TConstArrayView<FAssetCategoryPath> GetAssetCategories() const override
	{
		static const auto Categories = { FAssetCategoryPath(EAssetCategoryPaths::Media, NSLOCTEXT("AssetDefinition", "AssetDefinition_FileMediaSourceSubMenu", "Media Sources + Outputs"), ECategoryMenuType::Section) };
		return Categories;
	}
	// UAssetDefinition End
};
