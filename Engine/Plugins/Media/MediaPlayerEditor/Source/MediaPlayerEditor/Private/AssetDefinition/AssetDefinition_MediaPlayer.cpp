// Copyright Epic Games, Inc. All Rights Reserved.

#include "AssetDefinition_MediaPlayer.h"
#include "MediaPlayerEditorModule.h"
#include "Modules/ModuleManager.h"
#include "Toolkits/MediaPlayerEditorToolkit.h"

#define LOCTEXT_NAMESPACE "AssetDefinition_MediaPlayer"

EAssetCommandResult UAssetDefinition_MediaPlayer::OpenAssets(const FAssetOpenArgs& OpenArgs) const
{
	const EToolkitMode::Type Mode = OpenArgs.ToolkitHost.IsValid()
		? EToolkitMode::WorldCentric
		: EToolkitMode::Standalone;

	const TArray<UObject*> Objects = OpenArgs.LoadObjects<UObject>();
	for (auto ObjIt = Objects.CreateConstIterator(); ObjIt; ++ObjIt)
	{
		UMediaPlayer* MediaPlayer = Cast<UMediaPlayer>(*ObjIt);
		IMediaPlayerEditorModule* MediaPlayerEditorModule = FModuleManager::LoadModulePtr<IMediaPlayerEditorModule>("MediaPlayerEditor");

		if (MediaPlayer != nullptr && MediaPlayerEditorModule != nullptr)
		{
			TSharedPtr<ISlateStyle> MediaPlayerStyle = MediaPlayerEditorModule->GetStyle();
			if (MediaPlayerStyle.IsValid())
			{
				const TSharedRef<FMediaPlayerEditorToolkit> EditorToolkit = MakeShareable(new FMediaPlayerEditorToolkit(MediaPlayerStyle.ToSharedRef()));
				EditorToolkit->Initialize(MediaPlayer, Mode, OpenArgs.ToolkitHost);
			}
		}
	}

	return EAssetCommandResult::Handled;
}

#undef LOCTEXT_NAMESPACE
