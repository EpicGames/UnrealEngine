// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "AssetDefinitionDefault.h"
#include "MediaPlayer.h"
#include "AssetDefinition_MediaPlayer.generated.h"

UCLASS()
class UAssetDefinition_MediaPlayer : public UAssetDefinitionDefault
{
	GENERATED_BODY()
public:
	// UAssetDefinition Begin
	virtual FText GetAssetDisplayName() const override { return NSLOCTEXT("AssetDefinition", "AssetDefinition_MediaPlayer", "Media Player"); }
	virtual TSoftClassPtr<UObject> GetAssetClass() const override { return UMediaPlayer::StaticClass(); }
	virtual FLinearColor GetAssetColor() const override { return FColor::Red; }
	virtual TConstArrayView<FAssetCategoryPath> GetAssetCategories() const override
	{
		static const auto Categories = { FAssetCategoryPath(EAssetCategoryPaths::Media, NSLOCTEXT("AssetDefinition", "AssetDefinition_MediaPlayerSubMenu", "Basic"), ECategoryMenuType::Section) };
		return Categories;
	}
	virtual EAssetCommandResult OpenAssets(const FAssetOpenArgs& OpenArgs) const override;
	// UAssetDefinition End
};
