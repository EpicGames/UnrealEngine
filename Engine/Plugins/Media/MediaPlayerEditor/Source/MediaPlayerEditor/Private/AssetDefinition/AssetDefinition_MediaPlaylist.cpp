// Copyright Epic Games, Inc. All Rights Reserved.

#include "AssetDefinition_MediaPlaylist.h"
#include "MediaPlayerEditorModule.h"
#include "Modules/ModuleManager.h"
#include "Toolkits/MediaPlaylistEditorToolkit.h"

#define LOCTEXT_NAMESPACE "AssetDefinition_MediaPlaylist"

EAssetCommandResult UAssetDefinition_MediaPlaylist::OpenAssets(const FAssetOpenArgs& OpenArgs) const
{
	const EToolkitMode::Type Mode = OpenArgs.ToolkitHost.IsValid()
		? EToolkitMode::WorldCentric
		: EToolkitMode::Standalone;

	const TArray<UObject*> Objects = OpenArgs.LoadObjects<UObject>();
	for (auto ObjIt = Objects.CreateConstIterator(); ObjIt; ++ObjIt)
	{
		UMediaPlaylist* MediaPlaylist = Cast<UMediaPlaylist>(*ObjIt);
		IMediaPlayerEditorModule* MediaPlayerEditorModule = FModuleManager::LoadModulePtr<IMediaPlayerEditorModule>("MediaPlayerEditor");

		if (MediaPlaylist != nullptr && MediaPlayerEditorModule != nullptr)
		{
			TSharedPtr<ISlateStyle> MediaPlayerStyle = MediaPlayerEditorModule->GetStyle();
			if (MediaPlayerStyle.IsValid())
			{
				const TSharedRef<FMediaPlaylistEditorToolkit> EditorToolkit = MakeShareable(new FMediaPlaylistEditorToolkit(MediaPlayerStyle.ToSharedRef()));
				EditorToolkit->Initialize(MediaPlaylist, Mode, OpenArgs.ToolkitHost);
			}
		}
	}

	return EAssetCommandResult::Handled;
}

#undef LOCTEXT_NAMESPACE
