// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "AssetDefinitionDefault.h"
#include "MediaPlaylist.h"
#include "AssetDefinition_MediaPlaylist.generated.h"

UCLASS()
class UAssetDefinition_MediaPlaylist : public UAssetDefinitionDefault
{
	GENERATED_BODY()
public:
	// UAssetDefinition Begin
	virtual FText GetAssetDisplayName() const override { return NSLOCTEXT("AssetDefinition", "AssetDefinition_MediaPlaylist", "Media Playlist"); }
	virtual TSoftClassPtr<UObject> GetAssetClass() const override { return UMediaPlaylist::StaticClass(); }
	virtual FLinearColor GetAssetColor() const override { return FColor::Yellow; }
	virtual TConstArrayView<FAssetCategoryPath> GetAssetCategories() const override
	{
		static const auto Categories = { FAssetCategoryPath(EAssetCategoryPaths::Media, NSLOCTEXT("AssetDefinition", "AssetDefinition_MediaPlaylistSubMenu", "Basic"), ECategoryMenuType::Section) };
		return Categories;
	}
	virtual EAssetCommandResult OpenAssets(const FAssetOpenArgs& OpenArgs) const override;
	// UAssetDefinition End
};
