// Copyright Epic Games, Inc. All Rights Reserved.

#include "AssetDefinition/AssetDefinition_MediaSource.h"
#include "MediaPlayerEditorModule.h"
#include "Modules/ModuleManager.h"
#include "Toolkits/MediaSourceEditorToolkit.h"

#define LOCTEXT_NAMESPACE "AssetDefinition_MediaSource"

EAssetCommandResult UAssetDefinition_MediaSource::OpenAssets(const FAssetOpenArgs& OpenArgs) const
{
	EToolkitMode::Type Mode = OpenArgs.ToolkitHost.IsValid()
		? EToolkitMode::WorldCentric
		: EToolkitMode::Standalone;

	TArray<UObject*> Objects = OpenArgs.LoadObjects<UObject>();
	for (auto ObjIt = Objects.CreateConstIterator(); ObjIt; ++ObjIt)
	{
		UMediaSource* MediaSource = Cast<UMediaSource>(*ObjIt);

		if (MediaSource != nullptr)
		{
			IMediaPlayerEditorModule* MediaPlayerEditorModule = FModuleManager::LoadModulePtr<IMediaPlayerEditorModule>("MediaPlayerEditor");
			if (MediaPlayerEditorModule != nullptr)
			{
				TSharedPtr<ISlateStyle> MediaSourceStyle = MediaPlayerEditorModule->GetStyle();
				if (MediaSourceStyle.IsValid())
				{
					TSharedRef<FMediaSourceEditorToolkit> EditorToolkit = MakeShareable(new FMediaSourceEditorToolkit(MediaSourceStyle.ToSharedRef()));
					EditorToolkit->Initialize(MediaSource, Mode, OpenArgs.ToolkitHost);
				}
			}
		}
	}

	return EAssetCommandResult::Handled;
}

#undef LOCTEXT_NAMESPACE
