// Copyright Epic Games, Inc. All Rights Reserved.

#include "AssetDefinition_MediaTexture.h"

#include "AnimationEditorUtils.h"
#include "ContentBrowserMenuContexts.h"
#include "ToolMenus.h"
#include "Factories/MaterialFactoryNew.h"
#include "Interfaces/ITextureEditorModule.h"
#include "Materials/Material.h"
#include "Modules/ModuleManager.h"

#define LOCTEXT_NAMESPACE "AssetDefinition_MediaTexture"

EAssetCommandResult UAssetDefinition_MediaTexture::OpenAssets(const FAssetOpenArgs& OpenArgs) const
{
	const EToolkitMode::Type Mode = OpenArgs.ToolkitHost.IsValid() ? EToolkitMode::WorldCentric : EToolkitMode::Standalone;

	const TArray<UObject*> Objects = OpenArgs.LoadObjects<UObject>();
	for (auto ObjIt = Objects.CreateConstIterator(); ObjIt; ++ObjIt)
	{
		UTexture* Texture = Cast<UTexture>(*ObjIt);

		if (Texture != nullptr)
		{
			ITextureEditorModule* TextureEditorModule = &FModuleManager::LoadModuleChecked<ITextureEditorModule>("TextureEditor");
			TextureEditorModule->CreateTextureEditor(Mode, OpenArgs.ToolkitHost, Texture);
		}
	}

	return EAssetCommandResult::Handled;
}

#undef LOCTEXT_NAMESPACE
