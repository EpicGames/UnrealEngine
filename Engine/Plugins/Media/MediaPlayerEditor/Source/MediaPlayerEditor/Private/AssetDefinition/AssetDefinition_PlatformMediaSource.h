// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "PlatformMediaSource.h"
#include "AssetDefinition/AssetDefinition_MediaSource.h"
#include "AssetDefinition_PlatformMediaSource.generated.h"

UCLASS()
class UAssetDefinition_PlatformMediaSource : public UAssetDefinition_MediaSource
{
	GENERATED_BODY()
public:
	// UAssetDefinition Begin
	virtual FText GetAssetDisplayName() const override { return NSLOCTEXT("AssetDefinition", "AssetDefinition_PlatformMediaSource", "Platform Media Source"); }
	virtual TSoftClassPtr<UObject> GetAssetClass() const override { return UPlatformMediaSource::StaticClass(); }
	virtual TConstArrayView<FAssetCategoryPath> GetAssetCategories() const override
	{
		static const auto Categories = { FAssetCategoryPath(EAssetCategoryPaths::Media, NSLOCTEXT("AssetDefinition", "AssetDefinition_PlatformMediaSourceSubMenu", "Media Sources + Outputs"), ECategoryMenuType::Section) };
		return Categories;
	}
	// UAssetDefinition End
};
