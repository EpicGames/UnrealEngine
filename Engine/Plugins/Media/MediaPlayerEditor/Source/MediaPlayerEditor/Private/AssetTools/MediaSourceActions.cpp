// Copyright Epic Games, Inc. All Rights Reserved.

#include "AssetTools/MediaSourceActions.h"
#include "AssetRegistry/AssetData.h"
#include "MediaPlayerEditorModule.h"
#include "MediaSource.h"
#include "Toolkits/MediaSourceEditorToolkit.h"


#define LOCTEXT_NAMESPACE "AssetTypeActions"


/* FAssetTypeActions_Base interface
 *****************************************************************************/

bool FMediaSourceActions::CanFilter()
{
	return false;
}


FText FMediaSourceActions::GetAssetDescription(const struct FAssetData& AssetData) const
{
	FString Url = AssetData.GetTagValueRef<FString>("Url");
	if (Url.IsEmpty())
	{
		return LOCTEXT("AssetTypeActions_MediaSourceMissing", "Warning: Missing settings detected!");
	}

	FString Validate = AssetData.GetTagValueRef<FString>("Validate");
	bool bIsValidated = Validate.IsEmpty() || (Validate == TEXT("True"));
	if (!bIsValidated)
	{
		return LOCTEXT("AssetTypeActions_MediaSourceInvalid", "Warning: Invalid settings detected!");
	}

	return FText::GetEmpty();
}


uint32 FMediaSourceActions::GetCategories()
{
	return EAssetTypeCategories::Media;
}


FText FMediaSourceActions::GetName() const
{
	return FText::GetEmpty(); // let factory return sanitized class name
}


UClass* FMediaSourceActions::GetSupportedClass() const
{
	return UMediaSource::StaticClass();
}


FColor FMediaSourceActions::GetTypeColor() const
{
	return FColor::White;
}


TSharedPtr<class SWidget> FMediaSourceActions::GetThumbnailOverlay(const FAssetData& AssetData) const
{
	return nullptr;
/*	return SNew(SBorder)
		.BorderImage(FAppStyle::GetNoBrush())
		.Visibility(EVisibility::HitTestInvisible)
		.Padding(3.0f)
		.HAlign(HAlign_Right)
		.VAlign(VAlign_Top)
		[
			SNew(SImage)
				.Image(FAppStyle::GetBrush("Icons.Error"))
		];*/
}


void FMediaSourceActions::OpenAssetEditor(const TArray<UObject*>& InObjects, TSharedPtr<IToolkitHost> EditWithinLevelEditor)
{
	EToolkitMode::Type Mode = EditWithinLevelEditor.IsValid()
		? EToolkitMode::WorldCentric
		: EToolkitMode::Standalone;

	for (auto ObjIt = InObjects.CreateConstIterator(); ObjIt; ++ObjIt)
	{
		auto MediaSource = Cast<UMediaSource>(*ObjIt);

		if (MediaSource != nullptr)
		{
			IMediaPlayerEditorModule* MediaPlayerEditorModule = FModuleManager::LoadModulePtr<IMediaPlayerEditorModule>("MediaPlayerEditor");
			if (MediaPlayerEditorModule != nullptr)
			{
				TSharedPtr<ISlateStyle> Style = MediaPlayerEditorModule->GetStyle();

				TSharedRef<FMediaSourceEditorToolkit> EditorToolkit = MakeShareable(new FMediaSourceEditorToolkit(Style.ToSharedRef()));
				EditorToolkit->Initialize(MediaSource, Mode, EditWithinLevelEditor);
			}
		}
	}
}


#undef LOCTEXT_NAMESPACE
