// Copyright Epic Games, Inc. All Rights Reserved.

#include "Customizations/BaseMediaSourceCustomization.h"

#include "Styling/AppStyle.h"
#include "Modules/ModuleManager.h"
#include "PlatformInfo.h"

#include "DetailCategoryBuilder.h"
#include "DetailLayoutBuilder.h"
#include "DetailWidgetRow.h"
#include "IDetailPropertyRow.h"

#include "BaseMediaSource.h"
#include "IMediaPlayerFactory.h"
#include "IMediaModule.h"

#include "Framework/Commands/UIAction.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Layout/Margin.h"
#include "Textures/SlateIcon.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Layout/SGridPanel.h"
#include "Widgets/Input/SComboButton.h"
#include "Widgets/Text/STextBlock.h"


#define LOCTEXT_NAMESPACE "FBaseMediaSourceCustomization"


/* IDetailCustomization interface
 *****************************************************************************/

void FBaseMediaSourceCustomization::CustomizeDetails(IDetailLayoutBuilder& DetailBuilder)
{
	// customize 'Platforms' category
	IDetailCategoryBuilder& OverridesCategory = DetailBuilder.EditCategory("Platforms");
	{
		// PlatformPlayerNames
		PlatformPlayerNamesProperty = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(UBaseMediaSource, PlatformPlayerNames));
		{
			IDetailPropertyRow& PlayerNamesRow = OverridesCategory.AddProperty(PlatformPlayerNamesProperty);

			PlayerNamesRow
				.ShowPropertyButtons(false)
				.CustomWidget()
				.WholeRowContent()
				.MaxDesiredWidth(0.0f)
				[
					MakePlatformPlayerNamesValueWidget()
				];
		}
	}
}


/* FBaseMediaSourceCustomization implementation
 *****************************************************************************/

TSharedRef<SWidget> FBaseMediaSourceCustomization::MakePlatformPlayersMenu(const FString& IniPlatformName, const TArray<IMediaPlayerFactory*>& PlayerFactories)
{
	FMenuBuilder MenuBuilder(true, NULL);

	MenuBuilder.AddMenuEntry(
		LOCTEXT("AutoPlayer", "Automatic"),
		LOCTEXT("AutoPlayerTooltip", "Select a player automatically based on the media source"),
		FSlateIcon(),
		FUIAction(
		FExecuteAction::CreateLambda([this, IniPlatformName] { SetPlatformPlayerNamesValue(IniPlatformName, NAME_None); }),
			FCanExecuteAction()
		),
		NAME_None,
		EUserInterfaceActionType::Button
	);

	MenuBuilder.AddMenuSeparator();

	if (PlayerFactories.Num() == 0)
	{
		TSharedRef<SWidget> NoPlayersAvailableWidget = SNew(STextBlock)
			.ColorAndOpacity(FSlateColor::UseSubduedForeground())
			.Text(LOCTEXT("NoPlayerPluginsInstalled", "No media player plug-ins installed"));

		MenuBuilder.AddWidget(NoPlayersAvailableWidget, FText::GetEmpty(), true, false);
	}
	else
	{
		for (IMediaPlayerFactory* Factory : PlayerFactories)
		{
			const bool SupportsPlatform = Factory->GetSupportedPlatforms().Contains(*IniPlatformName);
			const FName PlayerName = Factory->GetPlayerName();

			MenuBuilder.AddMenuEntry(
				FText::Format(LOCTEXT("PlayerNameFormat", "{0} ({1})"), Factory->GetDisplayName(), FText::FromName(PlayerName)),
				FText::FromString(FString::Join(Factory->GetSupportedPlatforms(), TEXT(", "))),
				FSlateIcon(),
				FUIAction(
				FExecuteAction::CreateLambda([this, IniPlatformName, PlayerName] { SetPlatformPlayerNamesValue(IniPlatformName, PlayerName); }),
					FCanExecuteAction::CreateLambda([=] { return SupportsPlatform; })
				),
				NAME_None,
				EUserInterfaceActionType::Button
			);
		}
	}

	return MenuBuilder.MakeWidget();
}


TSharedRef<SWidget> FBaseMediaSourceCustomization::MakePlatformPlayerNamesValueWidget()
{
	// get registered player plug-ins
	auto MediaModule = FModuleManager::LoadModulePtr<IMediaModule>("Media");

	if (MediaModule == nullptr)
	{
		return SNew(STextBlock)
			.ColorAndOpacity(FSlateColor::UseSubduedForeground())
			.Text(LOCTEXT("NoPlayersAvailableLabel", "No players available"));
	}

	TArray<IMediaPlayerFactory*> PlayerFactories = MediaModule->GetPlayerFactories();
	{
		PlayerFactories.Sort([](IMediaPlayerFactory& A, IMediaPlayerFactory& B) -> bool {
			return (A.GetDisplayName().CompareTo(B.GetDisplayName()) < 0);
		});
	}

	// get available platforms
	const TArray<const FDataDrivenPlatformInfo*>& PlatformInfos = FDataDrivenPlatformInfoRegistry::GetSortedPlatformInfos(EPlatformInfoType::TruePlatformsOnly);

	// build value widget
	TSharedRef<SGridPanel> PlatformPanel = SNew(SGridPanel);

	for (int32 PlatformIndex = 0; PlatformIndex < PlatformInfos.Num(); ++PlatformIndex)
	{
		const FDataDrivenPlatformInfo* const Platform = PlatformInfos[PlatformIndex];

		// platform icon
		PlatformPanel->AddSlot(0, PlatformIndex)
			.VAlign(VAlign_Center)
			[
				SNew(SImage)
					.Image(FAppStyle::GetBrush(Platform->GetIconStyleName(EPlatformIconSize::Normal)))
			];

		// platform name
		PlatformPanel->AddSlot(1, PlatformIndex)
			.Padding(4.0f, 0.0f, 16.0f, 0.0f)
			.VAlign(VAlign_Center)
			[
				SNew(STextBlock)
					.Text(FText::FromName(Platform->IniPlatformName))
			];

		// player combo box
		PlatformPanel->AddSlot(2, PlatformIndex)
			.VAlign(VAlign_Center)
			[
				SNew(SComboButton)
					.ButtonContent()
					[
						SNew(STextBlock)
							.Text(this, &FBaseMediaSourceCustomization::HandlePlatformPlayersComboButtonText, Platform->IniPlatformName.ToString())
							.ToolTipText(LOCTEXT("PlatformPlayerComboButtonToolTipText", "Choose desired player for this platform"))
					]
					.ContentPadding(FMargin(6.0f, 2.0f))
					.MenuContent()
					[
						MakePlatformPlayersMenu(Platform->IniPlatformName.ToString(), PlayerFactories)
					]
			];
	}

	return PlatformPanel;
}


void FBaseMediaSourceCustomization::SetPlatformPlayerNamesValue(FString PlatformName, FName PlayerName)
{
	TArray<UObject*> OuterObjects;
	{
		PlatformPlayerNamesProperty->GetOuterObjects(OuterObjects);
	}

	for (auto Object : OuterObjects)
	{
		FName& OldPlayerName = Cast<UBaseMediaSource>(Object)->PlatformPlayerNames.FindOrAdd(PlatformName);;

		if (OldPlayerName != PlayerName)
		{
			Object->Modify(true);
			OldPlayerName = PlayerName;
		}
	}
}


/* FBaseMediaSourceCustomization callbacks
 *****************************************************************************/

FText FBaseMediaSourceCustomization::HandlePlatformPlayersComboButtonText(FString PlatformName) const
{
	TArray<UObject*> OuterObjects;
	{
		PlatformPlayerNamesProperty->GetOuterObjects(OuterObjects);
	}

	if ((OuterObjects.Num() == 0) || (OuterObjects[0] == nullptr))
	{
		return FText::GetEmpty();
	}

	FName PlayerName = Cast<UBaseMediaSource>(OuterObjects[0])->PlatformPlayerNames.FindRef(PlatformName);

	for (int32 ObjectIndex = 1; ObjectIndex < OuterObjects.Num(); ++ObjectIndex)
	{
		if ((OuterObjects[ObjectIndex] != nullptr) &&
			(Cast<UBaseMediaSource>(OuterObjects[ObjectIndex])->PlatformPlayerNames.FindRef(PlatformName) != PlayerName))
		{
			return NSLOCTEXT("PropertyEditor", "MultipleValues", "Multiple Values");
		}
	}

	if (PlayerName == NAME_None)
	{
		return LOCTEXT("AutomaticLabel", "Automatic");
	}

	return FText::FromName(PlayerName);
}


#undef LOCTEXT_NAMESPACE
