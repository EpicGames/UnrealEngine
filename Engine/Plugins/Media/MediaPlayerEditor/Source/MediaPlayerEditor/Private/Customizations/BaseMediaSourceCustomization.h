// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Widgets/SWidget.h"
#include "IDetailCustomization.h"

class IDetailLayoutBuilder;
class IMediaPlayerFactory;
class IPropertyHandle;

/**
 * Implements a details view customization for the UMediaSource class.
 */
class FBaseMediaSourceCustomization
	: public IDetailCustomization
{
public:

	//~ IDetailCustomization interface

	virtual void CustomizeDetails(IDetailLayoutBuilder& DetailBuilder) override;

public:

	/**
	 * Creates an instance of this class.
	 *
	 * @return The new instance.
	 */
	static TSharedRef<IDetailCustomization> MakeInstance()
	{
		return MakeShareable(new FBaseMediaSourceCustomization());
	}

protected:

	/**
	 * Create a player selection menu widget for the specified platform.
	 *
	 * @param IniPlatformName The INI name of the platform to create the menu for.
	 * @param PlayerFactories The available player factories.
	 * @return The menu widget.
	 */
	TSharedRef<SWidget> MakePlatformPlayersMenu(const FString& IniPlatformName, const TArray<IMediaPlayerFactory*>& PlayerFactories);

	/**
	 * Makes a widget for the DefaultPlayers property value.
	 *
	 * @return The widget.
	 */
	TSharedRef<SWidget> MakePlatformPlayerNamesValueWidget();

	/**
	 * Set the value of the PlatformPlayerNames property.
	 *
	 * @param PlayerName The name of the player to set.
	 */
	void SetPlatformPlayerNamesValue(FString PlatformName, FName PlayerName);

private:

	/** Callback for getting the text content of a platform player override combo button. */
	FText HandlePlatformPlayersComboButtonText(FString PlatformName) const;

private:

	/** Pointer to the DefaultPlayers property handle. */
	TSharedPtr<IPropertyHandle> PlatformPlayerNamesProperty;
};
