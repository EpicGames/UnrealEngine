// Copyright Epic Games, Inc. All Rights Reserved.

#include "Customizations/FileMediaSourceCustomization.h"
#include "DetailCategoryBuilder.h"
#include "DetailLayoutBuilder.h"
#include "DetailWidgetRow.h"
#include "FileMediaSource.h"
#include "IDetailPropertyRow.h"
#include "IMediaModule.h"
#include "Misc/Paths.h"
#include "Modules/ModuleManager.h"
#include "Styling/AppStyle.h"
#include "Styling/CoreStyle.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SFilePathPicker.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Text/STextBlock.h"


#define LOCTEXT_NAMESPACE "FFileMediaSourceCustomization"


/* IDetailCustomization interface
 *****************************************************************************/

void FFileMediaSourceCustomization::CustomizeDetails(IDetailLayoutBuilder& DetailBuilder)
{
	// customize 'File' category
	IDetailCategoryBuilder& FileCategory = DetailBuilder.EditCategory("File");
	{
		// FilePath
		FilePathProperty = DetailBuilder.GetProperty("FilePath");
		{
			IDetailPropertyRow& FilePathRow = FileCategory.AddProperty(FilePathProperty);

			FilePathRow
				.ShowPropertyButtons(false)
				.CustomWidget()
				.NameContent()
					[
						SNew(SHorizontalBox)

						+ SHorizontalBox::Slot()
							.AutoWidth()
							.VAlign(VAlign_Center)
							[
								SNew(STextBlock)
									.Font(IDetailLayoutBuilder::GetDetailFont())
									.Text(LOCTEXT("FilePathPropertyName", "File Path"))
									.ToolTipText(FilePathProperty->GetToolTipText())
							]

						+ SHorizontalBox::Slot()
							.FillWidth(1.0f)
							.HAlign(HAlign_Left)
							.VAlign(VAlign_Center)
							.Padding(4.0f, 0.0f, 0.0f, 0.0f)
							[
								SNew(SImage)
									.Image(FCoreStyle::Get().GetBrush("Icons.Warning"))
									.ToolTipText(LOCTEXT("FilePathWarning", "The selected media file will not get packaged, because its path points to a file outside the project's /Content/Movies/ directory."))
									.Visibility(this, &FFileMediaSourceCustomization::HandleFilePathWarningIconVisibility)
							]
					]
				.ValueContent()
					.MaxDesiredWidth(0.0f)
					.MinDesiredWidth(125.0f)
					[
						SNew(SFilePathPicker)
							.BrowseButtonImage(FAppStyle::GetBrush("PropertyWindow.Button_Ellipsis"))
							.BrowseButtonStyle(FAppStyle::Get(), "HoverHintOnly")
							.BrowseButtonToolTip(LOCTEXT("FilePathBrowseButtonToolTip", "Choose a file from this computer"))
							.BrowseDirectory(this, &FFileMediaSourceCustomization::HandleFilePathBrowseDirectory)
							.FilePath(this, &FFileMediaSourceCustomization::HandleFilePathPickerFilePath)
							.FileTypeFilter(this, &FFileMediaSourceCustomization::HandleFilePathPickerFileTypeFilter)
							.OnPathPicked(this, &FFileMediaSourceCustomization::HandleFilePathPickerPathPicked)
							.ToolTipText(LOCTEXT("FilePathToolTip", "The path to a media file on this computer"))
					];
		}
	}
}


FString FFileMediaSourceCustomization::GetResolvedFilePath() const
{
	TArray<UObject*> OuterObjects;
	FilePathProperty->GetOuterObjects(OuterObjects);
	for (UObject* OuterObject : OuterObjects)
	{
		if (const UFileMediaSource* Source = Cast<UFileMediaSource>(OuterObject))
		{
			// Using the media source to resolve the full path.
			return Source->GetFullPath();
		}
	}
	return FString();
}

/* FFileMediaSourceCustomization callbacks
 *****************************************************************************/

FString FFileMediaSourceCustomization::HandleFilePathBrowseDirectory() const
{
	const FString MediaPath = GetResolvedFilePath();
	return MediaPath.IsEmpty() ? (FPaths::ProjectContentDir() / TEXT("Movies")) : FPaths::GetPath(MediaPath);
}

FString FFileMediaSourceCustomization::HandleFilePathPickerFilePath() const
{
	FString FilePath;
	FilePathProperty->GetValue(FilePath);

	return FilePath;
}


FString FFileMediaSourceCustomization::HandleFilePathPickerFileTypeFilter() const
{
	FString Filter = TEXT("All files (*.*)|*.*");

	auto MediaModule = FModuleManager::GetModulePtr<IMediaModule>("Media");

	if (MediaModule == nullptr)
	{
		return Filter;
	}

/*	FMediaFileTypes FileTypes;
	MediaModule->GetSupportedFileTypes(FileTypes);

	if (FileTypes.Num() == 0)
	{
		return Filter;
	}

	FString AllExtensions;
	FString AllFilters;
			
	for (auto& Format : FileTypes)
	{
		if (!AllExtensions.IsEmpty())
		{
			AllExtensions += TEXT(";");
		}

		AllExtensions += TEXT("*.") + Format.Key;
		AllFilters += TEXT("|") + Format.Value.ToString() + TEXT(" (*.") + Format.Key + TEXT(")|*.") + Format.Key;
	}

	Filter = TEXT("All movie files (") + AllExtensions + TEXT(")|") + AllExtensions + TEXT("|") + Filter + AllFilters;
*/
	return Filter;
}

namespace UE::FileMediaSourceCustomization::Private
{
	/**
	 * Returns list of possible media base paths in order of priority. 
	 */
	const TArray<FString>& GetPossibleBasePaths()
	{
		static const TArray<FString> PossibleBasePaths
		{
			FPaths::ProjectContentDir(),
			FPaths::ProjectDir()
		};
		return PossibleBasePaths;
	}

	FString EnsureStartWithDotSlash(const FString& InPath)
	{
		if (!InPath.StartsWith(TEXT("./")))
		{
			return FPaths::Combine(TEXT("."), InPath);
		}
		return InPath;
	}

	/** Converts the given path relative to one of the possible base paths. */
	bool TryConvertAbsoluteToRelative(FString& InAbsolutePath)
	{
		FString ConvertedPath = InAbsolutePath;

		for (const FString& BasePath : GetPossibleBasePaths())
		{
			const FString FullBasePath = FPaths::ConvertRelativePathToFull(BasePath);
			if (FPaths::IsUnderDirectory(ConvertedPath, FullBasePath) && FPaths::MakePathRelativeTo(ConvertedPath, *FullBasePath)) 
			{
				InAbsolutePath = EnsureStartWithDotSlash(ConvertedPath);
				return true;
			}
		}
		return false;
	}
	
	/**
	 * Returns a sanitized path compliant with the path resolution rules
	 * of ImgMediaSource and FileMediaSource.
	 * 
	 * @param InPickedPath Picked path to an existing file.
	 * @return sanitized path 
	 */
	FString SanitizePickedPath(const FString& InPickedPath)
	{
		if (InPickedPath.IsEmpty())
		{
			return InPickedPath;	
		}

		FString NormalizedPath = InPickedPath.TrimStartAndEnd().Replace(TEXT("\""), TEXT(""));	
		FPaths::NormalizeDirectoryName(NormalizedPath);

		if (FPaths::IsRelative(NormalizedPath))
		{
			// 1- Try to resolve as relative to BaseDir...
			FString FullMediaPath = FPaths::ConvertRelativePathToFull(NormalizedPath);
			if (FPaths::FileExists(FullMediaPath))
			{
				// Convert absolute path to relative (if possible), leave absolute if not.
				TryConvertAbsoluteToRelative(FullMediaPath);
				return FullMediaPath;
			}

			// 2- Try to find under which possible base path this path is relative to.
			for (const FString& BasePath : GetPossibleBasePaths())
			{
				const FString FullBasePath = FPaths::ConvertRelativePathToFull(BasePath);
				FString CombinedMediaPath = FPaths::Combine(FullBasePath, NormalizedPath);
				
				if (FPaths::FileExists(CombinedMediaPath) && FPaths::MakePathRelativeTo(CombinedMediaPath, *FullBasePath))
				{
					return EnsureStartWithDotSlash(CombinedMediaPath);
				}
			}

			// 3- Couldn't find a base, leave as is, but make sure it has a ./
			return EnsureStartWithDotSlash(NormalizedPath);
		}

		// Convert absolute path to relative (if possible), leave absolute if not.
		TryConvertAbsoluteToRelative(NormalizedPath);
		return NormalizedPath;
	}
}

void FFileMediaSourceCustomization::HandleFilePathPickerPathPicked(const FString& PickedPath)
{
	if (FilePathProperty)
	{
		const FString SanitizedMediaPath = UE::FileMediaSourceCustomization::Private::SanitizePickedPath(PickedPath);
		FilePathProperty->SetValue(SanitizedMediaPath);
	}
}

EVisibility FFileMediaSourceCustomization::HandleFilePathWarningIconVisibility() const
{
	const FString FilePath = GetResolvedFilePath();

	if (FilePath.IsEmpty())
	{
		return EVisibility::Hidden;
	}

	const FString FullMoviesPath = FPaths::ConvertRelativePathToFull(FPaths::ProjectContentDir() / TEXT("Movies"));
	if (FPaths::IsUnderDirectory(FilePath, FullMoviesPath))
	{
		if (FPaths::FileExists(FilePath))
		{
			return EVisibility::Hidden;
		}

		// Path doesn't exist
		return EVisibility::Visible;
	}

	// file not inside Movies folder
	return EVisibility::Visible;
}


#undef LOCTEXT_NAMESPACE
