// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Layout/Visibility.h"
#include "IDetailCustomization.h"

class IDetailLayoutBuilder;
class IPropertyHandle;

/**
 * Implements a details view customization for the UFileMediaSource class.
 */
class FFileMediaSourceCustomization
	: public IDetailCustomization
{
public:

	//~ IDetailCustomization interface

	virtual void CustomizeDetails(IDetailLayoutBuilder& DetailBuilder) override;

public:

	/**
	 * Creates an instance of this class.
	 *
	 * @return The new instance.
	 */
	static TSharedRef<IDetailCustomization> MakeInstance()
	{
		return MakeShareable(new FFileMediaSourceCustomization());
	}

private:
	/** Returns the fully resolved path of the selected media file. */
	FString GetResolvedFilePath() const;

	/** Callback for getting the browse directory. */
	FString HandleFilePathBrowseDirectory() const;
	
	/** Callback for getting the selected path in the URL picker widget. */
	FString HandleFilePathPickerFilePath() const;

	/** Callback for getting the file type filter for the URL picker. */
	FString HandleFilePathPickerFileTypeFilter() const;

	/** Callback for picking a path in the URL picker. */
	void HandleFilePathPickerPathPicked(const FString& PickedPath);

	/** Callback for getting the visibility of warning icon for invalid URLs. */
	EVisibility HandleFilePathWarningIconVisibility() const;

private:

	/** Pointer to the FilePath property handle. */
	TSharedPtr<IPropertyHandle> FilePathProperty;
};
