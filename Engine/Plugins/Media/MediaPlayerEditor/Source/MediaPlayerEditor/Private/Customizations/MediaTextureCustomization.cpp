// Copyright Epic Games, Inc. All Rights Reserved.

#include "Customizations/MediaTextureCustomization.h"

#include "DetailCategoryBuilder.h"
#include "DetailLayoutBuilder.h"
#include "Engine/Texture.h"
#include "Widgets/SNullWidget.h"
#include "PropertyHandle.h"


/* IDetailCustomization interface
 *****************************************************************************/

void FMediaTextureCustomization::CustomizeDetails(IDetailLayoutBuilder& DetailBuilder)
{
	// customize 'Texture' category
	IDetailCategoryBuilder& MediaTextureCategory = DetailBuilder.EditCategory("MediaTexture");
	{
		MediaTextureCategory.AddProperty(GET_MEMBER_NAME_CHECKED(UTexture, Filter), UTexture::StaticClass());
	}

	DetailBuilder.HideCategory("Compression");
	DetailBuilder.HideCategory("Texture");
}
