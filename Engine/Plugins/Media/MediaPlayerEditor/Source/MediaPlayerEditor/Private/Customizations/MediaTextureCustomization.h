// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/WeakObjectPtr.h"
#include "IDetailCustomization.h"

class IDetailLayoutBuilder;
class IPropertyHandle;
class SWidget;


/**
 * Implements a details view customization for the UMediaTexture class.
 */
class FMediaTextureCustomization
	: public IDetailCustomization
{
public:

	//~ IDetailCustomization interface

	virtual void CustomizeDetails(IDetailLayoutBuilder& DetailBuilder) override;

public:

	/**
	 * Creates an instance of this class.
	 *
	 * @return The new instance.
	 */
	static TSharedRef<IDetailCustomization> MakeInstance()
	{
		return MakeShareable(new FMediaTextureCustomization());
	}
};
