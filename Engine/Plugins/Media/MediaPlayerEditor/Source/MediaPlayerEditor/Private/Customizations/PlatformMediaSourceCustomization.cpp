// Copyright Epic Games, Inc. All Rights Reserved.

#include "Customizations/PlatformMediaSourceCustomization.h"
#include "AssetRegistry/AssetData.h"
#include "MediaSource.h"
#include "PlatformMediaSource.h"
#include "Styling/SlateColor.h"
#include "Widgets/SWidget.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Styling/AppStyle.h"
#include "IDetailPropertyRow.h"
#include "Widgets/Text/STextBlock.h"
#include "DetailCategoryBuilder.h"
#include "DetailLayoutBuilder.h"
#include "IMediaModule.h"
#include "PlatformInfo.h"
#include "PropertyCustomizationHelpers.h"
#include "Modules/ModuleManager.h"
#include "Widgets/Layout/SGridPanel.h"
#include "Widgets/Images/SImage.h"



#define LOCTEXT_NAMESPACE "FPlatformMediaSourceCustomization"


/* IDetailCustomization interface
 *****************************************************************************/

void FPlatformMediaSourceCustomization::CustomizeDetails(IDetailLayoutBuilder& DetailBuilder)
{
	// customize 'Platforms' category
	IDetailCategoryBuilder& SourcesCategory = DetailBuilder.EditCategory("Sources");
	{
		// PlatformMediaSources
		PlatformMediaSourcesProperty = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(UPlatformMediaSource, PlatformMediaSources));
		{
			IDetailPropertyRow& PlatformMediaSourcesRow = SourcesCategory.AddProperty(PlatformMediaSourcesProperty);

			PlatformMediaSourcesRow
				.ShowPropertyButtons(false)
				.CustomWidget()
					.NameContent()
					[
						PlatformMediaSourcesProperty->CreatePropertyNameWidget()
					]
					.ValueContent()
					.MaxDesiredWidth(0.0f)
					[
						MakePlatformMediaSourcesValueWidget()
					];
		}
	}
}


/* FPlatformMediaSourceCustomization implementation
 *****************************************************************************/

TSharedRef<SWidget> FPlatformMediaSourceCustomization::MakePlatformMediaSourcesValueWidget()
{
	// get registered player plug-ins
	auto MediaModule = FModuleManager::LoadModulePtr<IMediaModule>("Media");

	if (MediaModule == nullptr)
	{
		return SNew(STextBlock)
			.ColorAndOpacity(FSlateColor::UseSubduedForeground())
			.Text(LOCTEXT("NoPlayersAvailableLabel", "No players available"));
	}

	const TArray<IMediaPlayerFactory*>& PlayerFactories = MediaModule->GetPlayerFactories();

	// get available platforms
	const TArray<const FDataDrivenPlatformInfo*>& PlatformInfos = FDataDrivenPlatformInfoRegistry::GetSortedPlatformInfos(EPlatformInfoType::TruePlatformsOnly);

	// build value widget
	TSharedRef<SGridPanel> PlatformPanel = SNew(SGridPanel);

	for (int32 PlatformIndex = 0; PlatformIndex < PlatformInfos.Num(); ++PlatformIndex)
	{
		const FDataDrivenPlatformInfo* const Platform = PlatformInfos[PlatformIndex];

		// platform icon
		PlatformPanel->AddSlot(0, PlatformIndex)
			.VAlign(VAlign_Center)
			[
				SNew(SImage)
					.Image(FAppStyle::GetBrush(Platform->GetIconStyleName(EPlatformIconSize::Normal)))
			];

		// platform name
		PlatformPanel->AddSlot(1, PlatformIndex)
			.Padding(4.0f, 0.0f, 16.0f, 0.0f)
			.VAlign(VAlign_Center)
			[
				SNew(STextBlock)
					.Text(FText::FromName(Platform->IniPlatformName))
			];

		// player combo box
		PlatformPanel->AddSlot(2, PlatformIndex)
			.VAlign(VAlign_Center)
			[
				SNew(SObjectPropertyEntryBox)
					.AllowedClass(UMediaSource::StaticClass())
					.AllowClear(true)
					.ObjectPath(this, &FPlatformMediaSourceCustomization::HandleMediaSourceEntryBoxObjectPath, Platform->IniPlatformName.ToString())
					.OnObjectChanged(this, &FPlatformMediaSourceCustomization::HandleMediaSourceEntryBoxChanged, Platform->IniPlatformName.ToString())
					.OnShouldFilterAsset(this, &FPlatformMediaSourceCustomization::HandleMediaSourceEntryBoxShouldFilterAsset)
			];
	}

	return PlatformPanel;
}


void FPlatformMediaSourceCustomization::SetPlatformMediaSourcesValue(FString PlatformName, UMediaSource* MediaSource)
{
	TArray<UObject*> OuterObjects;
	{
		PlatformMediaSourcesProperty->GetOuterObjects(OuterObjects);
	}

	for (auto Object : OuterObjects)
	{
		TObjectPtr<UMediaSource>& OldMediaSource = Cast<UPlatformMediaSource>(Object)->PlatformMediaSources.FindOrAdd(PlatformName);;

		if (OldMediaSource != MediaSource)
		{
			Object->Modify(true);
			OldMediaSource = MediaSource;
		}
	}
}


/* FPlatformMediaSourceCustomization callbacks
 *****************************************************************************/

void FPlatformMediaSourceCustomization::HandleMediaSourceEntryBoxChanged(const FAssetData& AssetData, FString PlatformName)
{
	TArray<UObject*> OuterObjects;
	{
		PlatformMediaSourcesProperty->GetOuterObjects(OuterObjects);
	}

	for (auto Object : OuterObjects)
	{
		UPlatformMediaSource* PlatformMediaSource = Cast<UPlatformMediaSource>(Object);
		if (PlatformMediaSource != nullptr)
		{
			TObjectPtr<UMediaSource>& OldMediaSource = PlatformMediaSource->PlatformMediaSources.FindOrAdd(PlatformName);;

			if (OldMediaSource != AssetData.GetAsset())
			{
				Object->Modify(true);
				OldMediaSource = Cast<UMediaSource>(AssetData.GetAsset());
				PlatformMediaSource->GenerateThumbnail();
			}
		}
	}
}


FString FPlatformMediaSourceCustomization::HandleMediaSourceEntryBoxObjectPath(FString PlatformName) const
{
	TArray<UObject*> OuterObjects;
	{
		PlatformMediaSourcesProperty->GetOuterObjects(OuterObjects);
	}

	if (OuterObjects.Num() == 0)
	{
		return FString();
	}

	UMediaSource* MediaSource = Cast<UPlatformMediaSource>(OuterObjects[0])->PlatformMediaSources.FindRef(PlatformName);

	for (int32 ObjectIndex = 1; ObjectIndex < OuterObjects.Num(); ++ObjectIndex)
	{
		if (Cast<UPlatformMediaSource>(OuterObjects[ObjectIndex])->PlatformMediaSources.FindRef(PlatformName) != MediaSource)
		{
			return FString();
		}
	}

	if (MediaSource == nullptr)
	{
		return FString();
	}

	return MediaSource->GetPathName();
}


bool FPlatformMediaSourceCustomization::HandleMediaSourceEntryBoxShouldFilterAsset(const FAssetData& AssetData)
{
	// Don't allow nesting platform media sources.
	UClass* AssetClass = FindObject<UClass>(AssetData.AssetClassPath);
	return AssetClass->IsChildOf(UPlatformMediaSource::StaticClass());
}


#undef LOCTEXT_NAMESPACE
