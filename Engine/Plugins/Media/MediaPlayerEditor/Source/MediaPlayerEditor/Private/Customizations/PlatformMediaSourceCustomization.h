// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "IDetailCustomization.h"

struct FAssetData;
class IDetailLayoutBuilder;
class IPropertyHandle;
class SWidget;
class UMediaSource;

/**
 * Implements a details view customization for the UPlatformMediaSource class.
 */
class FPlatformMediaSourceCustomization
	: public IDetailCustomization
{
public:

	//~ IDetailCustomization interface

	virtual void CustomizeDetails(IDetailLayoutBuilder& DetailBuilder) override;

public:

	/**
	 * Creates an instance of this class.
	 *
	 * @return The new instance.
	 */
	static TSharedRef<IDetailCustomization> MakeInstance()
	{
		return MakeShareable(new FPlatformMediaSourceCustomization());
	}

protected:

	/**
	 * Makes a widget for the DefaultPlayers property value.
	 *
	 * @return The widget.
	 */
	TSharedRef<SWidget> MakePlatformMediaSourcesValueWidget();

	/**
	 * Set the value of the PlatformMediaSources property.
	 *
	 * @param MediaSource The media source to set.
	 */
	void SetPlatformMediaSourcesValue(FString PlatformName, UMediaSource* MediaSource);

private:

	/** Callback for when a per-platform media source property changed. */
	void HandleMediaSourceEntryBoxChanged(const FAssetData& AssetData, FString PlatformName);

	/** Callback for getting the currently selected object in a per-platform media source property. */
	FString HandleMediaSourceEntryBoxObjectPath(FString PlatformName) const;

	/** Callback for filtering media source assets. */
	bool HandleMediaSourceEntryBoxShouldFilterAsset(const FAssetData& AssetData);

private:

	/** Pointer to the DefaultPlayers property handle. */
	TSharedPtr<IPropertyHandle> PlatformMediaSourcesProperty;
};
