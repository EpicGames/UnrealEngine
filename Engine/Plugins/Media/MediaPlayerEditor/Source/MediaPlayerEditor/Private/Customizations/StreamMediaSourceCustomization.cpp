// Copyright Epic Games, Inc. All Rights Reserved.

#include "StreamMediaSourceCustomization.h"

#include "DetailCategoryBuilder.h"
#include "DetailLayoutBuilder.h"
#include "DetailWidgetRow.h"
#include "IMediaCaptureSupport.h"
#include "MediaCaptureSupport.h"
#include "MediaPlayerEditorModule.h"
#include "StreamMediaSource.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Modules/ModuleManager.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SComboButton.h"

#define LOCTEXT_NAMESPACE "FStreamMediaSourceCustomization"

void FStreamMediaSourceCustomization::CustomizeDetails(IDetailLayoutBuilder& DetailBuilder)
{
	// Manual sort for the stream media source's property categories to ensure that 'Stream' category apppears on top
	DetailBuilder.SortCategories([](const TMap<FName, IDetailCategoryBuilder*>& CategoryMap)
	{
		for (const TPair<FName, IDetailCategoryBuilder*>& Pair : CategoryMap)
		{
			int32 SortOrder = Pair.Value->GetSortOrder();
			const FName CategoryName = Pair.Key;

			if (CategoryName != "Stream")
			{
				SortOrder += 10;
			}

			Pair.Value->SetSortOrder(SortOrder);
		}
	});

	StreamUrlHandle = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(UStreamMediaSource, StreamUrl));
	if (IDetailPropertyRow* Row = DetailBuilder.EditDefaultProperty(StreamUrlHandle))
	{
		IMediaPlayerEditorModule* MediaPlayerEditorModule = FModuleManager::LoadModulePtr<IMediaPlayerEditorModule>("MediaPlayerEditor");
		TSharedPtr<ISlateStyle> Style = MediaPlayerEditorModule->GetStyle();
		
		TSharedPtr<SWidget> NameWidget;
		TSharedPtr<SWidget> ValueWidget;
		FDetailWidgetRow& CustomWidget = Row->CustomWidget();
		
		Row->GetDefaultWidgets(NameWidget, ValueWidget);
		
		CustomWidget
		.NameContent()
		[
			NameWidget.ToSharedRef()
		];

		CustomWidget.ValueContent()
		.MinDesiredWidth(125.0f)
		.MaxDesiredWidth(600.0f)
		[
			SNew(SHorizontalBox)

			+SHorizontalBox::Slot()
			[
				ValueWidget.ToSharedRef()
			]

			+SHorizontalBox::Slot()
			.AutoWidth()
			[
				SNew(SComboButton)
				.ComboButtonStyle(FAppStyle::Get(), "SimpleComboButtonWithIcon")
				.OnGetMenuContent(this, &FStreamMediaSourceCustomization::GetDiscoverUrlMenu)
				.ButtonContent()
				[
					SNew(SImage)
					.Image(Style->GetBrush("MediaPlayerEditor.DiscoverUrl"))
					.ColorAndOpacity(FSlateColor::UseForeground())
				]
			]
		];
	}
}

TSharedRef<SWidget> FStreamMediaSourceCustomization::GetDiscoverUrlMenu()
{
	constexpr bool bCloseMenuAfterSelection = true;
	FMenuBuilder MenuBuilder(bCloseMenuAfterSelection, nullptr);

	MenuBuilder.BeginSection("CaptureDevicesSection", LOCTEXT("CaptureDevicesSection", "Capture Devices"));
	{
		MenuBuilder.AddSubMenu(
			LOCTEXT("AudioMenuLabel", "Audio"),
			LOCTEXT("AudioMenuTooltip", "Available audio capture devices"),
			FNewMenuDelegate::CreateSP(this, &FStreamMediaSourceCustomization::GetAudioCaptureDevicesSubmenu),
			false,
			FSlateIcon()
		);

		MenuBuilder.AddSubMenu(
			LOCTEXT("VideoMenuLabel", "Video"),
			LOCTEXT("VideoMenuTooltip", "Available video capture devices"),
			FNewMenuDelegate::CreateSP(this, &FStreamMediaSourceCustomization::GetVideoCaptureDevicesSubmenu),
			false,
			FSlateIcon()
		);
	}
	MenuBuilder.EndSection();
	
	return MenuBuilder.MakeWidget();
}

void FStreamMediaSourceCustomization::GetAudioCaptureDevicesSubmenu(FMenuBuilder& MenuBuilder)
{
	TArray<FMediaCaptureDeviceInfo> DeviceInfos;
	MediaCaptureSupport::EnumerateAudioCaptureDevices(DeviceInfos);
	MakeCaptureDeviceMenu(DeviceInfos, MenuBuilder);
}

void FStreamMediaSourceCustomization::GetVideoCaptureDevicesSubmenu(FMenuBuilder& MenuBuilder)
{
	TArray<FMediaCaptureDeviceInfo> DeviceInfos;
	MediaCaptureSupport::EnumerateVideoCaptureDevices(DeviceInfos);
	MakeCaptureDeviceMenu(DeviceInfos, MenuBuilder);
}

void FStreamMediaSourceCustomization::MakeCaptureDeviceMenu(TArray<FMediaCaptureDeviceInfo>& DeviceInfos, FMenuBuilder& MenuBuilder)
{
	for (const auto& DeviceInfo : DeviceInfos)
	{
		MenuBuilder.AddMenuEntry(
			DeviceInfo.DisplayName,
			FText::GetEmpty(),
			FSlateIcon(),
			FUIAction(FExecuteAction::CreateSP(this, &FStreamMediaSourceCustomization::SetStreamMediaSourceUrl, DeviceInfo.Url)),
			NAME_None,
			EUserInterfaceActionType::Button
		);
	}
}

void FStreamMediaSourceCustomization::SetStreamMediaSourceUrl(FString InUrl)
{
	if (!StreamUrlHandle.IsValid() || !StreamUrlHandle->IsValidHandle())
	{
		return;
	}

	StreamUrlHandle->SetValue(InUrl);
}

#undef LOCTEXT_NAMESPACE
