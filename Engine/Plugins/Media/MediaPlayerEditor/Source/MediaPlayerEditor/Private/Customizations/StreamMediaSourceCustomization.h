// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IDetailCustomization.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"

class IPropertyHandle;
struct FMediaCaptureDeviceInfo;
class SWidget;

/** Customization for stream media sources, primarily used to add a 'discover URL' dropdown to the stream URL property */
class FStreamMediaSourceCustomization : public IDetailCustomization
{
public:
	static TSharedRef<IDetailCustomization> MakeInstance()
	{
		return MakeShareable(new FStreamMediaSourceCustomization());
	}

	//~ IDetailCustomization interface
	
public:
	virtual void CustomizeDetails(IDetailLayoutBuilder& DetailBuilder) override;
	
	//~ End IDetailCustomization interface

private:
	/** Creates the dropdown menu for the 'discover URL' combo button */
	TSharedRef<SWidget> GetDiscoverUrlMenu();
	
	/** Fills the menu builder with options for found audio source URLs */
	void GetAudioCaptureDevicesSubmenu(FMenuBuilder& MenuBuilder);
	
	/** Fills the menu builder with options for found video source URLs */
	void GetVideoCaptureDevicesSubmenu(FMenuBuilder& MenuBuilder);

	/** Fills the menu builder with entries for all provided device infos */
	void MakeCaptureDeviceMenu(TArray<FMediaCaptureDeviceInfo>& DeviceInfos, FMenuBuilder& MenuBuilder);

	/** Raised when a user selects a URL entry in the 'discover URLs' dropdown, sets the stream URL to the selected URL */
	void SetStreamMediaSourceUrl(FString InUrl);

private:
	/** Cached property handle for UStreamMediaSource.StreamUrl property */
	TSharedPtr<IPropertyHandle> StreamUrlHandle;
};
