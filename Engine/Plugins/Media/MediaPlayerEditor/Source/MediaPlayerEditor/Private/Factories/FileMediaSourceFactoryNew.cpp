// Copyright Epic Games, Inc. All Rights Reserved.

#include "Factories/FileMediaSourceFactoryNew.h"
#include "AssetTypeCategories.h"
#include "FileMediaSource.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(FileMediaSourceFactoryNew)


/* UFileMediaSourceFactoryNew structors
 *****************************************************************************/

UFileMediaSourceFactoryNew::UFileMediaSourceFactoryNew(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	SupportedClass = UFileMediaSource::StaticClass();
	bCreateNew = true;
	bEditAfterNew = true;
}


/* UFactory interface
 *****************************************************************************/

UObject* UFileMediaSourceFactoryNew::FactoryCreateNew(UClass* InClass, UObject* InParent, FName InName, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn)
{
	return NewObject<UFileMediaSource>(InParent, InClass, InName, Flags);
}


uint32 UFileMediaSourceFactoryNew::GetMenuCategories() const
{
	return EAssetTypeCategories::Media;
}


bool UFileMediaSourceFactoryNew::ShouldShowInNewMenu() const
{
	return true;
}

