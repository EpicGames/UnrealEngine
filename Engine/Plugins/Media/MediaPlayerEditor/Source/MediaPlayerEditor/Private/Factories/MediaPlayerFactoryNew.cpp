// Copyright Epic Games, Inc. All Rights Reserved.

#include "MediaPlayerFactoryNew.h"

#include "AssetToolsModule.h"
#include "Editor.h"
#include "Styling/AppStyle.h"
#include "EngineAnalytics.h"
#include "IAssetTools.h"
#include "Input/Reply.h"
#include "Misc/PackageName.h"
#include "Modules/ModuleManager.h"
#include "Layout/Visibility.h"
#include "MediaPlayer.h"
#include "MediaTexture.h"
#include "Styling/SlateTypes.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/SWindow.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Layout/SUniformGridPanel.h"
#include "Widgets/Text/STextBlock.h"

#include "Factories/MediaTextureFactoryNew.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MediaPlayerFactoryNew)


#define LOCTEXT_NAMESPACE "UMediaPlayerFactoryNew"


/* Local helpers
 *****************************************************************************/

class SMediaPlayerFactoryDialog
	: public SCompoundWidget
{
public:

	SLATE_BEGIN_ARGS(SMediaPlayerFactoryDialog) { }
	SLATE_END_ARGS()

	/** Construct this widget. */
	void Construct(const FArguments& InArgs, FMediaPlayerFactoryNewOptions& InOptions, TSharedRef<SWindow> InWindow)
	{
		Options = &InOptions;
		Window = InWindow;

		ChildSlot
		[
			SNew(SBorder)
				.Visibility(EVisibility::Visible)
				.BorderImage(FAppStyle::GetBrush("Menu.Background"))
				[
					SNew(SVerticalBox)

					+ SVerticalBox::Slot()
						.FillHeight(1)
						.VAlign(VAlign_Top)
						[
							SNew(SBorder)
								.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
								.Padding(4.0f)
								.Content()
								[
									SNew(SVerticalBox)

									+ SVerticalBox::Slot()
										[
											SNew(STextBlock)
												.Text(LOCTEXT("CreateAdditionalAssetsLabel", "Additional assets to create and link to the Media Player:"))
										]

									+ SVerticalBox::Slot()
										.Padding(0.0f, 6.0f, 0.0f, 0.0f)
										[
											SNew(SCheckBox)
												.IsChecked(Options->CreateVideoTexture ? ECheckBoxState::Checked : ECheckBoxState::Unchecked)
												.OnCheckStateChanged_Lambda([this](ECheckBoxState CheckBoxState) {
													Options->CreateVideoTexture = (CheckBoxState == ECheckBoxState::Checked);
												})
												.Content()
												[
													SNew(STextBlock)
														.Text(LOCTEXT("CreateVideoTextureLabel", "Video output MediaTexture asset"))
												]
										]
								]
						]

					+ SVerticalBox::Slot()
						.AutoHeight()
						.HAlign(HAlign_Right)
						.VAlign(VAlign_Bottom)
						.Padding(8)
						[
							SNew(SUniformGridPanel)
								.SlotPadding(FAppStyle::GetMargin("StandardDialog.SlotPadding"))
								.MinDesiredSlotWidth(FAppStyle::GetFloat("StandardDialog.MinDesiredSlotWidth"))
								.MinDesiredSlotHeight(FAppStyle::GetFloat("StandardDialog.MinDesiredSlotHeight"))
				
							+ SUniformGridPanel::Slot(0, 0)
								[
									SNew(SButton)
										.HAlign(HAlign_Center)
										.ContentPadding(FAppStyle::GetMargin("StandardDialog.ContentPadding"))
										.OnClicked_Lambda([this]() -> FReply { CloseDialog(true); return FReply::Handled(); })
										.Text(LOCTEXT("OkButtonLabel", "OK"))
								]
					
							+ SUniformGridPanel::Slot(1, 0)
								[
									SNew(SButton)
										.HAlign(HAlign_Center)
										.ContentPadding(FAppStyle::GetMargin("StandardDialog.ContentPadding"))
										.OnClicked_Lambda([this]() -> FReply { CloseDialog(false); return FReply::Handled(); })
										.Text(LOCTEXT("CancelButtonLabel", "Cancel"))
								]
						]
				]
		];
	}

protected:

	void CloseDialog(bool InOkClicked)
	{
		Options->OkClicked = InOkClicked;

		if (Window.IsValid())
		{
			Window.Pin()->RequestDestroyWindow();
		}
	}

private:

	FMediaPlayerFactoryNewOptions* Options;
	TWeakPtr<SWindow> Window;
};


/* UMediaPlayerFactoryNew structors
 *****************************************************************************/

UMediaPlayerFactoryNew::UMediaPlayerFactoryNew( const FObjectInitializer& ObjectInitializer )
	: Super(ObjectInitializer)
{
	SupportedClass = UMediaPlayer::StaticClass();
	bCreateNew = true;
	bEditAfterNew = true;
}


/* UFactory interface
 *****************************************************************************/

bool UMediaPlayerFactoryNew::ConfigureProperties()
{
	Options.CreateVideoTexture = false;
	Options.OkClicked = false;

	TSharedRef<SWindow> Window = SNew(SWindow)
		.Title(LOCTEXT("CreateMediaPlayerDialogTitle", "Create Media Player"))
		.ClientSize(FVector2D(400, 160))
		.SupportsMinimize(false)
		.SupportsMaximize(false);

	Window->SetContent(SNew(SMediaPlayerFactoryDialog, Options, Window));
	GEditor->EditorAddModalWindow(Window);

	return Options.OkClicked;
}

/**
 * @EventName MediaFramework.CreateNewMediaPlayer
 * @Trigger Triggered when a media player asset is created.
 * @Type Client
 * @Owner MediaIO Team
 */
UObject* UMediaPlayerFactoryNew::FactoryCreateNew(UClass* InClass, UObject* InParent, FName InName, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn)
{
	auto NewMediaPlayer = NewObject<UMediaPlayer>(InParent, InClass, InName, Flags);

	if ((NewMediaPlayer != nullptr) && Options.CreateVideoTexture)
	{
		IAssetTools& AssetTools = FModuleManager::Get().LoadModuleChecked<FAssetToolsModule>("AssetTools").Get();
		const FString ParentName = InParent->GetOutermost()->GetName();

		FString OutAssetName;
		FString OutPackageName;

		if (FEngineAnalytics::IsAvailable())
		{
			FEngineAnalytics::GetProvider().RecordEvent(TEXT("MediaFramework.CreateNewMediaPlayer"));
		}

		AssetTools.CreateUniqueAssetName(ParentName, TEXT("_Video"), OutPackageName, OutAssetName);
		const FString PackagePath = FPackageName::GetLongPackagePath(OutPackageName);
		auto Factory = NewObject<UMediaTextureFactoryNew>();
		auto VideoTexture = Cast<UMediaTexture>(AssetTools.CreateAsset(OutAssetName, PackagePath, UMediaTexture::StaticClass(), Factory));

		if (VideoTexture != nullptr)
		{
			VideoTexture->SetDefaultMediaPlayer(NewMediaPlayer);
		}
	}

	return NewMediaPlayer;
}


uint32 UMediaPlayerFactoryNew::GetMenuCategories() const
{
	return EAssetTypeCategories::Media;
}


bool UMediaPlayerFactoryNew::ShouldShowInNewMenu() const
{
	return true;
}


#undef LOCTEXT_NAMESPACE

