// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Factories/Factory.h"
#include "MediaPlayerFactoryNew.generated.h"

/**
 * Options for UMediaPlayerFactoryNew.
 */
struct FMediaPlayerFactoryNewOptions
{
	bool CreateVideoTexture;
	bool OkClicked;
};


/**
 * Implements a factory for UMediaPlayer objects.
 */
UCLASS(hidecategories=Object)
class UMediaPlayerFactoryNew
	: public UFactory
{
	GENERATED_UCLASS_BODY()

public:

	//~ UFactory Interface

	virtual bool ConfigureProperties() override;
	virtual UObject* FactoryCreateNew(UClass* InClass, UObject* InParent, FName InName, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn) override;
	virtual uint32 GetMenuCategories() const override;
	virtual bool ShouldShowInNewMenu() const override;

private:

	FMediaPlayerFactoryNewOptions Options;
};
