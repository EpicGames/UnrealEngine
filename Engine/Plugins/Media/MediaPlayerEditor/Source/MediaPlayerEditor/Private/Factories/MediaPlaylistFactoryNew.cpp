// Copyright Epic Games, Inc. All Rights Reserved.

#include "Factories/MediaPlaylistFactoryNew.h"
#include "AssetTypeCategories.h"
#include "MediaPlaylist.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MediaPlaylistFactoryNew)


/* UMediaPlaylist structors
 *****************************************************************************/

UMediaPlaylistFactoryNew::UMediaPlaylistFactoryNew(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	SupportedClass = UMediaPlaylist::StaticClass();
	bCreateNew = true;
	bEditAfterNew = true;
}


/* UFactory interface
 *****************************************************************************/

UObject* UMediaPlaylistFactoryNew::FactoryCreateNew(UClass* InClass, UObject* InParent, FName InName, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn)
{
	return NewObject<UMediaPlaylist>(InParent, InClass, InName, Flags);
}


uint32 UMediaPlaylistFactoryNew::GetMenuCategories() const
{
	return EAssetTypeCategories::Media;
}


bool UMediaPlaylistFactoryNew::ShouldShowInNewMenu() const
{
	return true;
}

