// Copyright Epic Games, Inc. All Rights Reserved.

#include "Factories/MediaTextureFactoryNew.h"
#include "AssetTypeCategories.h"
#include "EngineAnalytics.h"
#include "MediaTexture.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MediaTextureFactoryNew)


/* UMediaTextureFactoryNew structors
 *****************************************************************************/

UMediaTextureFactoryNew::UMediaTextureFactoryNew(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	SupportedClass = UMediaTexture::StaticClass();
	bCreateNew = true;
	bEditAfterNew = true;
}


/* UFactory overrides
 *****************************************************************************/


/**
 * @EventName MediaFramework.CreateNewMediaTexture
 * @Trigger Triggered when a media texture asset is created.
 * @Type Client
 * @Owner MediaIO Team
 */
UObject* UMediaTextureFactoryNew::FactoryCreateNew(UClass* InClass, UObject* InParent, FName InName, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn)
{
	if (FEngineAnalytics::IsAvailable())
	{
		FEngineAnalytics::GetProvider().RecordEvent(TEXT("MediaFramework.CreateNewMediaTexture"));
	}
	
	auto MediaTexture = NewObject<UMediaTexture>(InParent, InClass, InName, Flags);

	if (MediaTexture != nullptr)
	{
		MediaTexture->UpdateResource();
	}

	return MediaTexture;
}


uint32 UMediaTextureFactoryNew::GetMenuCategories() const
{
	return EAssetTypeCategories::Media | EAssetTypeCategories::Textures;
}


bool UMediaTextureFactoryNew::ShouldShowInNewMenu() const
{
	return true;
}

