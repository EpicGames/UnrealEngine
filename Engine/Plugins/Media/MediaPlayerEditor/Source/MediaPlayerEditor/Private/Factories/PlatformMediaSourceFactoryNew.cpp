// Copyright Epic Games, Inc. All Rights Reserved.

#include "Factories/PlatformMediaSourceFactoryNew.h"
#include "AssetTypeCategories.h"
#include "PlatformMediaSource.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PlatformMediaSourceFactoryNew)


/* UPlatformMediaSourceFactoryNew structors
 *****************************************************************************/

UPlatformMediaSourceFactoryNew::UPlatformMediaSourceFactoryNew(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	SupportedClass = UPlatformMediaSource::StaticClass();
	bCreateNew = true;
	bEditAfterNew = true;
}


/* UFactory interface
 *****************************************************************************/

UObject* UPlatformMediaSourceFactoryNew::FactoryCreateNew(UClass* InClass, UObject* InParent, FName InName, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn)
{
	return NewObject<UPlatformMediaSource>(InParent, InClass, InName, Flags);
}


uint32 UPlatformMediaSourceFactoryNew::GetMenuCategories() const
{
	return EAssetTypeCategories::Media;
}


bool UPlatformMediaSourceFactoryNew::ShouldShowInNewMenu() const
{
	return true;
}

