// Copyright Epic Games, Inc. All Rights Reserved.

#include "Factories/StreamMediaSourceFactoryNew.h"
#include "AssetTypeCategories.h"
#include "StreamMediaSource.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(StreamMediaSourceFactoryNew)


/* UStreamMediaSourceFactoryNew structors
 *****************************************************************************/

UStreamMediaSourceFactoryNew::UStreamMediaSourceFactoryNew(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	SupportedClass = UStreamMediaSource::StaticClass();
	bCreateNew = true;
	bEditAfterNew = true;
}


/* UFactory overrides
 *****************************************************************************/

UObject* UStreamMediaSourceFactoryNew::FactoryCreateNew(UClass* InClass, UObject* InParent, FName InName, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn)
{
	return NewObject<UStreamMediaSource>(InParent, InClass, InName, Flags);
}


uint32 UStreamMediaSourceFactoryNew::GetMenuCategories() const
{
	return EAssetTypeCategories::Media;
}


bool UStreamMediaSourceFactoryNew::ShouldShowInNewMenu() const
{
	return true;
}

