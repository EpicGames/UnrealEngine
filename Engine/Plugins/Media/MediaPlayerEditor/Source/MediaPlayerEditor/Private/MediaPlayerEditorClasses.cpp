// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreMinimal.h"
#include "Shared/MediaPlayerEditorSettings.h"


UMediaPlayerEditorSettings::UMediaPlayerEditorSettings(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
	, ShowTextOverlays(true)
	, ViewportScale(EMediaPlayerEditorScale::Fit)
{ }
