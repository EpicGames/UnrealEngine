// Copyright Epic Games, Inc. All Rights Reserved.

#include "MediaPlayerEditorModule.h"

#include "ComponentVisualizer.h"
#include "ContentBrowserMenuContexts.h"
#include "Modules/ModuleManager.h"
#include "AssetToolsModule.h"
#include "UObject/UObjectHash.h"
#include "UObject/UObjectIterator.h"
#include "Editor.h"
#include "Editor/UnrealEdEngine.h"
#include "IAssetTypeActions.h"
#include "PropertyEditorModule.h"
#include "ThumbnailRendering/ThumbnailManager.h"
#include "ThumbnailRendering/TextureThumbnailRenderer.h"
#include "Toolkits/AssetEditorToolkit.h"
#include "UnrealEdGlobals.h"

#include "BaseMediaSource.h"
#include "FileMediaSource.h"
#include "StreamMediaSource.h"
#include "IMediaAssetsModule.h"
#include "MediaPlayer.h"
#include "MediaSoundComponent.h"
#include "Shared/MediaSourceRenderer.h"
#include "MediaTexture.h"
#include "PlatformMediaSource.h"

#include "AssetTools/MediaSourceActions.h"

#include "Customizations/BaseMediaSourceCustomization.h"
#include "Customizations/FileMediaSourceCustomization.h"
#include "Customizations/StreamMediaSourceCustomization.h"
#include "Customizations/MediaTextureCustomization.h"
#include "Customizations/PlatformMediaSourceCustomization.h"

#include "Models/MediaPlayerEditorCommands.h"
#include "Shared/MediaPlayerEditorStyle.h"
#include "Shared/MediaSourceThumbnailRenderer.h"
#include "Visualizers/MediaSoundComponentVisualizer.h"
#include "ToolMenus.h"
#include "Widgets/SMediaPlayerEditorMedia.h"
#include "Widgets/SMediaPlayerSlider.h"

#include "MediaPlayerEditorLog.h"


#define LOCTEXT_NAMESPACE "FMediaPlayerEditorModule"

DEFINE_LOG_CATEGORY(LogMediaPlayerEditor);

/**
 * Implements the MediaPlayerEditor module.
 */
class FMediaPlayerEditorModule
	: public IHasMenuExtensibility
	, public IHasToolBarExtensibility
	, public IMediaPlayerEditorModule
{
public:

	//~ IHasMenuExtensibility interface

	virtual TSharedPtr<FExtensibilityManager> GetMenuExtensibilityManager() override
	{
		return MenuExtensibilityManager;
	}

public:

	//~ IHasToolBarExtensibility interface

	virtual TSharedPtr<FExtensibilityManager> GetToolBarExtensibilityManager() override
	{
		return ToolBarExtensibilityManager;
	}

public:

	//~ IMediaPlayerEditorModule interface
	virtual TSharedPtr<ISlateStyle> GetStyle()
	{
		return Style;
	}

	TSharedRef<IMediaPlayerSlider> CreateMediaPlayerSliderWidget(UMediaPlayer* InMediaPlayer, const FSliderStyle& InStyle)
	{
		TWeakObjectPtr<UMediaPlayer> MediaPlayerWeak = InMediaPlayer;
		return CreateMediaPlayerSliderWidget(MakeArrayView(&MediaPlayerWeak, 1), InStyle);
	}

	virtual TSharedRef<IMediaPlayerSlider> CreateMediaPlayerSliderWidget(const TArrayView<TWeakObjectPtr<UMediaPlayer>> InMediaPlayers, const FSliderStyle& InStyle) override
	{
		return SNew(SMediaPlayerSlider, InMediaPlayers)
		.Style(&InStyle);
	}

	//~ IModuleInterface interface

	virtual void StartupModule() override
	{
		BaseMediaSourceName = UBaseMediaSource::StaticClass()->GetFName();
		FileMediaSourceName = UFileMediaSource::StaticClass()->GetFName();
		StreamMediaSourceName = UStreamMediaSource::StaticClass()->GetFName();
		MediaSoundComponentName = UMediaSoundComponent::StaticClass()->GetFName();
		MediaTextureName = UMediaTexture::StaticClass()->GetFName();
		PlatformMediaSourceName = UPlatformMediaSource::StaticClass()->GetFName();

		Style = MakeShareable(new FMediaPlayerEditorStyle());

		FMediaPlayerEditorCommands::Register();

		SMediaPlayerEditorMedia::RegisterMenus();

		RegisterAssetTools();
		RegisterCustomizations();
		RegisterEditorDelegates();
		RegisterMenuExtensions();
		RegisterThumbnailRenderers();
		RegisterVisualizers();
		ExtendContentMenu();
	}

	virtual void ShutdownModule() override
	{
		UnregisterAssetTools();
		UnregisterCustomizations();
		UnregisterEditorDelegates();
		UnregisterMenuExtensions();
		UnregisterThumbnailRenderers();
		UnregisterVisualizers();
	}

protected:

	/** Registers asset tool actions. */
	void RegisterAssetTools()
	{
		IAssetTools& AssetTools = FModuleManager::LoadModuleChecked<FAssetToolsModule>("AssetTools").Get();

		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		RegisterAssetTypeAction(AssetTools, MakeShareable(new FMediaSourceActions()));
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

	/**
	 * Registers a single asset type action.
	 *
	 * @param AssetTools The asset tools object to register with.
	 * @param Action The asset type action to register.
	 */
	void RegisterAssetTypeAction( IAssetTools& AssetTools, TSharedRef<IAssetTypeActions> Action )
	{
		AssetTools.RegisterAssetTypeActions(Action);
		RegisteredAssetTypeActions.Add(Action);
	}

	/** Unregisters asset tool actions. */
	void UnregisterAssetTools()
	{
		FAssetToolsModule* AssetToolsModule = FModuleManager::GetModulePtr<FAssetToolsModule>("AssetTools");

		if (AssetToolsModule != nullptr)
		{
			IAssetTools& AssetTools = AssetToolsModule->Get();

			for (auto Action : RegisteredAssetTypeActions)
			{
				AssetTools.UnregisterAssetTypeActions(Action);
			}
		}
	}

protected:

	/** Register details view customizations. */
	void RegisterCustomizations()
	{
		FPropertyEditorModule& PropertyModule = FModuleManager::LoadModuleChecked<FPropertyEditorModule>("PropertyEditor");
		{
			PropertyModule.RegisterCustomClassLayout(FileMediaSourceName, FOnGetDetailCustomizationInstance::CreateStatic(&FFileMediaSourceCustomization::MakeInstance));
			PropertyModule.RegisterCustomClassLayout(StreamMediaSourceName, FOnGetDetailCustomizationInstance::CreateStatic(&FStreamMediaSourceCustomization::MakeInstance));
			PropertyModule.RegisterCustomClassLayout(MediaTextureName, FOnGetDetailCustomizationInstance::CreateStatic(&FMediaTextureCustomization::MakeInstance));
			PropertyModule.RegisterCustomClassLayout(PlatformMediaSourceName, FOnGetDetailCustomizationInstance::CreateStatic(&FPlatformMediaSourceCustomization::MakeInstance));
			PropertyModule.RegisterCustomClassLayout(BaseMediaSourceName, FOnGetDetailCustomizationInstance::CreateStatic(&FBaseMediaSourceCustomization::MakeInstance));
		}
	}

	/** Unregister details view customizations. */
	void UnregisterCustomizations()
	{
		FPropertyEditorModule& PropertyModule = FModuleManager::LoadModuleChecked<FPropertyEditorModule>("PropertyEditor");
		{
			PropertyModule.UnregisterCustomClassLayout(BaseMediaSourceName);
			PropertyModule.UnregisterCustomClassLayout(FileMediaSourceName);
			PropertyModule.UnregisterCustomClassLayout(StreamMediaSourceName);
			PropertyModule.UnregisterCustomClassLayout(MediaTextureName);
			PropertyModule.UnregisterCustomClassLayout(PlatformMediaSourceName);
		}
	}

protected:

	/** Register Editor delegates. */
	void RegisterEditorDelegates()
	{
		FEditorDelegates::BeginPIE.AddRaw(this, &FMediaPlayerEditorModule::HandleEditorBeginPIE);
		FEditorDelegates::EndPIE.AddRaw(this, &FMediaPlayerEditorModule::HandleEditorEndPIE);
		FEditorDelegates::PausePIE.AddRaw(this, &FMediaPlayerEditorModule::HandleEditorPausePIE);
		FEditorDelegates::ResumePIE.AddRaw(this, &FMediaPlayerEditorModule::HandleEditorResumePIE);
	}

	/** Unregister Editor delegates. */
	void UnregisterEditorDelegates()
	{
		FEditorDelegates::BeginPIE.RemoveAll(this);
		FEditorDelegates::EndPIE.RemoveAll(this);
		FEditorDelegates::PausePIE.RemoveAll(this);
		FEditorDelegates::ResumePIE.RemoveAll(this);
	}

protected:

	/** Registers main menu and tool bar menu extensions. */
	void RegisterMenuExtensions()
	{
		MenuExtensibilityManager = MakeShareable(new FExtensibilityManager);
		ToolBarExtensibilityManager = MakeShareable(new FExtensibilityManager);
	}

	/** Unregisters main menu and tool bar menu extensions. */
	void UnregisterMenuExtensions()
	{
		MenuExtensibilityManager.Reset();
		ToolBarExtensibilityManager.Reset();
	}

protected:

	/** Registers asset thumbnail renderers .*/
	void RegisterThumbnailRenderers()
	{
		IMediaAssetsModule* MediaAssetsModule = FModuleManager::LoadModulePtr<IMediaAssetsModule>("MediaAssets");
		if (MediaAssetsModule != nullptr)
		{
			MediaAssetsModule->RegisterCreateMediaSourceRenderer(
				IMediaAssetsModule::FOnCreateMediaSourceRenderer::CreateRaw(this,
					&FMediaPlayerEditorModule::CreateMediaSourceRenderer));
		}

		UThumbnailManager::Get().RegisterCustomRenderer(UMediaTexture::StaticClass(), UTextureThumbnailRenderer::StaticClass());
		UThumbnailManager::Get().RegisterCustomRenderer(UMediaSource::StaticClass(), UMediaSourceThumbnailRenderer::StaticClass());
	}

	/** Unregisters all asset thumbnail renderers. */
	void UnregisterThumbnailRenderers()
	{
		if (UObjectInitialized())
		{
			UThumbnailManager::Get().UnregisterCustomRenderer(UMediaSource::StaticClass());
			UThumbnailManager::Get().UnregisterCustomRenderer(UMediaTexture::StaticClass());

			IMediaAssetsModule* MediaAssetsModule = FModuleManager::GetModulePtr<IMediaAssetsModule>("MediaAssets");
			if (MediaAssetsModule != nullptr)
			{
				MediaAssetsModule->UnregisterCreateMediaSourceRenderer();
			}
		}
	}

	/** Creates an object that implements IMediaSourceRendererInterface. */
	UObject* CreateMediaSourceRenderer()
	{
		return NewObject<UMediaSourceRenderer>(GetTransientPackage());
	}

protected:

	/** Register all component visualizers. */
	void RegisterVisualizers()
	{
		if (GUnrealEd != nullptr)
		{
			RegisterVisualizer(*GUnrealEd, MediaSoundComponentName, MakeShared<FMediaSoundComponentVisualizer>());
		}
	}

	/**
	 * Register a component visualizer.
	 *
	 * @param ComponentClassName The name of the component class to register.
	 * @param Visualizer The component visualizer to register.
	 */
	void RegisterVisualizer(UUnrealEdEngine& UnrealEdEngine, const FName& ComponentClassName, const TSharedRef<FComponentVisualizer>& Visualizer)
	{
		UnrealEdEngine.RegisterComponentVisualizer(ComponentClassName, Visualizer);
		Visualizer->OnRegister();
	}

	/** Unregister all component visualizers. */
	void UnregisterVisualizers()
	{
		if (GUnrealEd != nullptr)
		{
			GUnrealEd->UnregisterComponentVisualizer(MediaSoundComponentName);
		}
	}

private:

	void ExtendContentMenu()
	{
		UToolMenu* Menu = UE::ContentBrowser::ExtendToolMenu_AssetContextMenu(UMediaSource::StaticClass());

		FToolMenuSection& Section = Menu->FindOrAddSection("GetAssetActions");
		Section.AddDynamicEntry(NAME_None, FNewToolMenuSectionDelegate::CreateLambda([](FToolMenuSection& InSection)
		{
			if (const UContentBrowserAssetContextMenuContext* Context = UContentBrowserAssetContextMenuContext::FindContextWithAssets(InSection))
			{
				const TAttribute<FText> Label = LOCTEXT("GenerateThumbnail", "Generate Thumbnail");
				const TAttribute<FText> ToolTip = LOCTEXT("GenerateThumbnail_Tooltip", "Generate a thumbnail for this asset.");
				const FSlateIcon Icon = FSlateIcon(FAppStyle::GetAppStyleSetName(), "AssetEditor.SaveThumbnail");;

				FToolUIAction UIAction;
				UIAction.ExecuteAction = FToolMenuExecuteAction::CreateLambda([](const FToolMenuContext& InContext)
				{
					const UContentBrowserAssetContextMenuContext* Context = UContentBrowserAssetContextMenuContext::FindContextWithAssets(InContext);

					for (UMediaSource* MediaSource : Context->LoadSelectedObjects<UMediaSource>())
					{
						MediaSource->GenerateThumbnail();
						MediaSource->MarkPackageDirty();
					}
				});
				InSection.AddMenuEntry("Media_GenerateThumbnail", Label, ToolTip, Icon, UIAction);
			}
		}));
	}

	void HandleEditorBeginPIE(bool bIsSimulating)
	{
		for (TObjectIterator<UMediaPlayer> It; It; ++It)
		{
			UMediaPlayer* Player = *It;
			if (Player->AffectedByPIEHandling)
			{
				Player->Close();
			}
		}
	}

	void HandleEditorEndPIE(bool bIsSimulating)
	{
		for (TObjectIterator<UMediaPlayer> It; It; ++It)
		{
			UMediaPlayer* Player = *It;
			if (Player->AffectedByPIEHandling)
			{
				(*It)->Close();
			}
		}
	}

	void HandleEditorPausePIE(bool bIsSimulating)
	{
		for (TObjectIterator<UMediaPlayer> It; It; ++It)
		{
			UMediaPlayer* Player = *It;
			if (Player->AffectedByPIEHandling)
			{
				(*It)->PausePIE();
			}
		}
	}

	void HandleEditorResumePIE(bool bIsSimulating)
	{
		for (TObjectIterator<UMediaPlayer> It; It; ++It)
		{
			UMediaPlayer* Player = *It;
			if (Player->AffectedByPIEHandling)
			{
				(*It)->ResumePIE();
			}
		}
	}

private:

	/** Holds the menu extensibility manager. */
	TSharedPtr<FExtensibilityManager> MenuExtensibilityManager;

	/** The collection of registered asset type actions. */
	TArray<TSharedRef<IAssetTypeActions>> RegisteredAssetTypeActions;

	/** Holds the plug-ins style set. */
	TSharedPtr<ISlateStyle> Style;

	/** Holds the tool bar extensibility manager. */
	TSharedPtr<FExtensibilityManager> ToolBarExtensibilityManager;

	/** Class names. */
	FName BaseMediaSourceName;
	FName FileMediaSourceName;
	FName StreamMediaSourceName;
	FName MediaSoundComponentName;
	FName MediaTextureName;
	FName PlatformMediaSourceName;
};


IMPLEMENT_MODULE(FMediaPlayerEditorModule, MediaPlayerEditor);


#undef LOCTEXT_NAMESPACE
