// Copyright Epic Games, Inc. All Rights Reserved.

#include "Models/MediaPlayerEditorCommands.h"


#define LOCTEXT_NAMESPACE "FMediaPlayerEditorCommands"

const FMediaPlayerEditorCommands& FMediaPlayerEditorCommands::GetExternal()
{
	if (!IsRegistered())
	{
		Register();
	}

	return Get();
}

void FMediaPlayerEditorCommands::RegisterCommands()
{
	UI_COMMAND(CloseMedia, "Close", "Close the currently opened media", EUserInterfaceActionType::Button, FInputChord(EKeys::Escape));
	UI_COMMAND(GenerateThumbnail, "Thumbnail", "Generate a thumbnail", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(OpenMedia, "Open", "Open the current media", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(PauseMedia, "Pause", "Pause media playback", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(PlayMedia, "Play", "Start media playback", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(PlayReverseMedia, "Play in Reverse", "Start media playback in reverse. Not widely supported.", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(NextMedia, "Next", "Jump to next item in the play list", EUserInterfaceActionType::Button, FInputChord(EKeys::L, EModifierKey::Shift));
	UI_COMMAND(PreviousMedia, "Prev", "Jump to previous item in the play list", EUserInterfaceActionType::Button, FInputChord(EKeys::J, EModifierKey::Shift));
	
	UI_COMMAND(
		RewindMedia,
		"Rewind",
		"Rewind the media to the beginning",
		EUserInterfaceActionType::Button,
		FInputChord(EKeys::J, EModifierKey::Control)
	);

	UI_COMMAND(
		ReverseMedia, 
		"Reverse", 
		"Play media in reverse at ever increasing speed. Not widely supported.", 
		EUserInterfaceActionType::Button, 
		FInputChord(EKeys::J, EModifierKey::Alt)
	);

	UI_COMMAND(
		StepBackwardMedia,
		"Step Backward",
		"Step the media backward 1 frame if it is paused",
		EUserInterfaceActionType::Button,
		FInputChord(EKeys::J)
	);

	UI_COMMAND(
		TogglePlayPauseMedia,
		"Toggle Play/Pause",
		"Toggles between media play and pause",
		EUserInterfaceActionType::ToggleButton,
		FInputChord(EKeys::K),
		FInputChord(EKeys::SpaceBar)
	);

	UI_COMMAND(
		TogglePlayReversePauseMedia,
		"Toggle Reverse Play/Pause",
		"Toggles between media reverse play and pause. Not widely supported.",
		EUserInterfaceActionType::ToggleButton,
		FInputChord(EKeys::K, EModifierKey::Control),
		FInputChord(EKeys::SpaceBar, EModifierKey::Control)
	);

	UI_COMMAND(
		ForwardMedia, 
		"Fast Forward", 
		"Play media forward at ever increasing speed", 
		EUserInterfaceActionType::Button, 
		FInputChord(EKeys::L, EModifierKey::Alt)
	);

	UI_COMMAND(
		StepForwardMedia,
		"Step Forward",
		"Step the media forward 1 frame if it is paused",
		EUserInterfaceActionType::Button,
		FInputChord(EKeys::L)
	);

	UI_COMMAND(
		JumpToEndMedia,
		"Jumo to End", 
		"Jump to the end of the media. Will jump back to the start if looping is enabled.", 
		EUserInterfaceActionType::Button, 
		FInputChord(EKeys::L, EModifierKey::Control)
	);

	UI_COMMAND(
		ToggledRedTextureChannel,
		"R",
		"Toggles the red texture channel on and onn",
		EUserInterfaceActionType::ToggleButton,
		FInputChord(EKeys::R)
	);

	UI_COMMAND(
		ToggledGreenTextureChannel,
		"G",
		"Toggles the green texture channel on and onn",
		EUserInterfaceActionType::ToggleButton,
		FInputChord(EKeys::G)
	);

	UI_COMMAND(
		ToggledBlueTextureChannel,
		"B",
		"Toggles the blue texture channel on and onn",
		EUserInterfaceActionType::ToggleButton,
		FInputChord(EKeys::B)
	);

	UI_COMMAND(
		ToggledAlphaTextureChannel,
		"A",
		"Toggles the alpha texture channel on and onn",
		EUserInterfaceActionType::ToggleButton,
		FInputChord(EKeys::A)
	);
}


#undef LOCTEXT_NAMESPACE
