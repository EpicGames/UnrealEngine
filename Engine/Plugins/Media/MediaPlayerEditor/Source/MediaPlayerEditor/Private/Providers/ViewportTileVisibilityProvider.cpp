// Copyright Epic Games, Inc. All Rights Reserved.

#include "Providers/ViewportTileVisibilityProvider.h"

#include "Assets/MediaSequenceInfo.h"
#include "Math/UnrealMathUtility.h"

namespace UE::MediaPlayerEditor
{

void FViewportTileVisibilityProvider::SetDisplaySizePx(FIntPoint InSizePx)
{
	bool bChanged = false;
	{
		FScopeLock Lock(&Mutex);
		if (DisplaySizePx != InSizePx)
		{
			DisplaySizePx = InSizePx;
			bChanged = true;
		}
	}
	// Broadcast outside the mutex.
	if (bChanged)
	{
		BroadcastOnInputsChanged();
	}
}

void FViewportTileVisibilityProvider::GatherVisibleTiles(const FMediaSequenceInfo& InSequenceInfo,
	FMediaTileVisibilityRequest& OutRequest) const
{
	if (InSequenceInfo.Dim.X <= 0 || InSequenceInfo.Dim.Y <= 0
		|| InSequenceInfo.NumMipLevels <= 0)
	{
		return;
	}

	FIntPoint LocalDisplaySizePx;
	{
		FScopeLock Lock(&Mutex);
		LocalDisplaySizePx = DisplaySizePx;
	}

	// Pre-paint guard: until the consumer's first Tick has pushed a real display size,
	// emit no tiles. Treating the missing size as "full source" would resolve mip 0 with
	// every tile visible and spike-load mip 0 for one frame on open. The loader handles
	// an empty visibility set the same as "user looking away" - safe.
	if (LocalDisplaySizePx.X <= 0 || LocalDisplaySizePx.Y <= 0)
	{
		return;
	}

	const int32 MaxLevel = InSequenceInfo.NumMipLevels - 1;
	const float RatioX = static_cast<float>(InSequenceInfo.Dim.X) / static_cast<float>(LocalDisplaySizePx.X);
	const float RatioY = static_cast<float>(InSequenceInfo.Dim.Y) / static_cast<float>(LocalDisplaySizePx.Y);
	// Min ratio = the dimension where the display has the highest pixel density. That's
	// the axis driving auto-LOD; using max would over-pick mip on an aspect-mismatched
	// display.
	const float Ratio = FMath::Min(RatioX, RatioY);
	const float LOD = (Ratio > 1.0f) ? FMath::Log2(Ratio) : 0.0f;
	const int32 FloorMip = FMath::Clamp(FMath::FloorToInt32(LOD), 0, MaxLevel);
	const int32 CeilMip = FMath::Clamp(FMath::CeilToInt32(LOD), 0, MaxLevel);

	OutRequest.VisibleTiles.Add(FloorMip,
		FMediaTileSelection::CreateForTargetMipLevel(InSequenceInfo.Dim,
			InSequenceInfo.TilingDescription.TileSize, FloorMip, /*bDefaultVisibility=*/ true));
	if (CeilMip != FloorMip)
	{
		OutRequest.VisibleTiles.Add(CeilMip,
			FMediaTileSelection::CreateForTargetMipLevel(InSequenceInfo.Dim,
				InSequenceInfo.TilingDescription.TileSize, CeilMip, /*bDefaultVisibility=*/ true));
	}
}

bool FViewportTileVisibilityProvider::IsAlive() const
{
	// Lifetime is bound to the owning widget via the TSharedPtr the widget holds; the
	// resolver tracks this provider weakly and prunes it once the widget drops its
	// strong reference.
	return true;
}

} // namespace UE::MediaPlayerEditor
