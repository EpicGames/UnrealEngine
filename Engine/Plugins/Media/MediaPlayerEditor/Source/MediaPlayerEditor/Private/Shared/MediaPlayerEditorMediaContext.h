// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "ToolMenus.h"

#include "MediaPlayerEditorMediaContext.generated.h"

class SMediaPlayerEditorMedia;

UCLASS(MinimalAPI, BlueprintType)
class UMediaPlayerEditorMediaContext : public UToolMenuContextBase
{
	GENERATED_BODY()

public:	

	void InitContext(UObject* InSelectedAsset, FName InStyleSetName = NAME_None)
	{
		SelectedAsset = InSelectedAsset;
		StyleSetName = InStyleSetName;
	}

	UPROPERTY(BlueprintReadWrite, Category = "Editor Menus")
	TObjectPtr<UObject> SelectedAsset;

	UPROPERTY(BlueprintReadWrite, Category = "Editor Menus")
	FName StyleSetName;

	TWeakPtr<SMediaPlayerEditorMedia> MediaPlayerEditorMedia;
};
