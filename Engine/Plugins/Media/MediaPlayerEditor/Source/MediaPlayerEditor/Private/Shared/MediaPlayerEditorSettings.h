// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "MediaPlayerEditorSettings.generated.h"


/** Options for scaling the viewport's video texture. */
UENUM()
enum class EMediaPlayerEditorScale : uint8
{
	/** Stretch non-uniformly to fill the viewport. */
	Fill,

	/** Scale uniformly, preserving aspect ratio. */
	Fit,

	/** Do not stretch or scale. */
	Original
};

/* Background types */
UENUM()
enum class EMediaPlayerEditorBackground : uint8
{
	Black,
	Checkered
};

/* Timer display units */
UENUM()
enum class EMediaPlayerEditorTimerUnit : uint8
{
	/** Display timer in HH:MM:SS format. */
	Timecode,
	/** Display timer as integer frame number. */
	FrameNumber
};

UCLASS(config=EditorPerProjectUserSettings)
class UMediaPlayerEditorSettings
	: public UObject
{
	GENERATED_UCLASS_BODY()

public:

	/** The name of the desired native media player to use for playback. */
	UPROPERTY(config, EditAnywhere, Category=Viewer)
	FName DesiredPlayerName;

	/** Whether to display overlay texts. */
	UPROPERTY(config, EditAnywhere, Category=Viewer)
	bool ShowTextOverlays;

	/** Whether to display the playback timer as frame number or timecode. */
	UPROPERTY(config, EditAnywhere, Category=Viewer)
	EMediaPlayerEditorTimerUnit TimerUnit = EMediaPlayerEditorTimerUnit::Timecode;

	/** How the video viewport should be scaled. */
	UPROPERTY(config, EditAnywhere, Category=Viewer)
	EMediaPlayerEditorScale ViewportScale;

	/** Which background to use on the video background. */
	UPROPERTY(config, EditAnywhere, Category = Viewer)
	EMediaPlayerEditorBackground MediaBackground = EMediaPlayerEditorBackground::Black;
};
