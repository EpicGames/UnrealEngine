// Copyright Epic Games, Inc. All Rights Reserved.

#include "MediaPlayerEditorStyle.h"

#include "Brushes/SlateBorderBrush.h"
#include "Brushes/SlateBoxBrush.h"
#include "Brushes/SlateImageBrush.h"
#include "Fonts/SlateFontInfo.h"
#include "Math/Vector2D.h"
#include "Misc/Paths.h"
#include "Styling/SlateStyleMacros.h"
#include "Styling/SlateStyleRegistry.h"
#include "Styling/SlateTypes.h"
#include "Styling/CoreStyle.h"

/* FMediaPlayerEditorStyle structors
 *****************************************************************************/

FMediaPlayerEditorStyle::FMediaPlayerEditorStyle()
	: FSlateStyleSet("MediaPlayerEditorStyle")
{
	const FVector2D Icon8x8(8.0f, 8.0f);
	const FVector2D Icon12x12(12.0f, 12.0f);
	const FVector2D Icon16x16(16.0f, 16.0f);
	const FVector2D Icon20x20(20.0f, 20.0f);
	const FVector2D Icon40x40(40.0f, 40.0f);

	SetContentRoot(FPaths::EnginePluginsDir() / TEXT("Media/MediaPlayerEditor/Content"));
	SetCoreContentRoot(FPaths::EngineContentDir() / TEXT("Editor/Slate"));

	// buttons
	Set("MediaPlayerEditor.SourceButton", new IMAGE_BRUSH("btn_source_12x", Icon12x12));
	Set("MediaPlayerEditor.GoButton", new IMAGE_BRUSH("btn_go_12x", Icon12x12));
	Set("MediaPlayerEditor.ReloadButton", new IMAGE_BRUSH("btn_reload_12x", Icon12x12));
	Set("MediaPlayerEditor.SettingsButton", new IMAGE_BRUSH("btn_settings_16x", Icon12x12));

	Set<FButtonStyle>("MediaPlayerEditor.MediaControlButton", FButtonStyle()
		.SetNormal(FSlateNoResource())
		.SetDisabled(FSlateNoResource())
		.SetHovered(FSlateRoundedBoxBrush(FLinearColor(0.2f, 0.2f, 0.2f, 0.5f), 3.f, Icon20x20))
		.SetPressed(FSlateRoundedBoxBrush(FLinearColor(0.1f, 0.1f, 0.1f, 0.5f), 3.f, Icon20x20))
		.SetNormalPadding(FMargin(2.f, 2.f, 2.f, 2.f))
		.SetPressedPadding(FMargin(2.f, 2.f, 2.f, 2.f)));

	// misc
	Set("MediaPlayerEditor.DragDropBorder", new BOX_BRUSH("border_dragdrop", 0.5f));
	Set("MediaPlayerEditor.MediaSourceOpened", new IMAGE_BRUSH("mediasource_opened", Icon8x8));
	Set("MediaPlayerEditor.DiscoverUrl", new CORE_IMAGE_BRUSH_SVG( "Starship/EditorViewport/globe", Icon16x16 ));

	// tabs
	Set("MediaPlayerEditor.Tabs.Info", new IMAGE_BRUSH("tab_info_16x", Icon16x16));
	Set("MediaPlayerEditor.Tabs.Media", new IMAGE_BRUSH("tab_media_16x", Icon16x16));
	Set("MediaPlayerEditor.Tabs.Player", new IMAGE_BRUSH("tab_player_16x", Icon16x16));
	Set("MediaPlayerEditor.Tabs.Playlist", new IMAGE_BRUSH("tab_playlist_16x", Icon16x16));
	Set("MediaPlayerEditor.Tabs.Stats", new IMAGE_BRUSH("tab_stats_16x", Icon16x16));

	// toolbar icons
	Set("MediaPlayerEditor.GenerateThumbnail", new CORE_IMAGE_BRUSH_SVG("Starship/AssetEditors/SaveThumbnail", Icon20x20));
	//Set("MediaPlayerEditor.ForwardMedia", new CORE_IMAGE_BRUSH_SVG("Sequencer/PlaybackControls/SaveThumbnail", Icon20x20));
	Set("MediaPlayerEditor.NextMedia", new CORE_IMAGE_BRUSH_SVG("Starship/Common/NextArrow", Icon20x20));
	Set("MediaPlayerEditor.PauseMedia", new CORE_IMAGE_BRUSH_SVG("Sequencer/PlaybackControls/PlayControlsPause", Icon20x20));
	Set("MediaPlayerEditor.PlayMedia", new CORE_IMAGE_BRUSH_SVG("Sequencer/PlaybackControls/PlayControlsPlayForward", Icon20x20));
	Set("MediaPlayerEditor.PlayReverseMedia", new CORE_IMAGE_BRUSH_SVG("Sequencer/PlaybackControls/PlayControlsPlayReverse", Icon20x20));
	Set("MediaPlayerEditor.PreviousMedia", new CORE_IMAGE_BRUSH_SVG("Starship/Common/PreviousArrow", Icon20x20));
	//Set("MediaPlayerEditor.ReverseMedia", new CORE_IMAGE_BRUSH_SVG("Sequencer/PlaybackControls/SaveThumbnail", Icon20x20));
	Set("MediaPlayerEditor.RewindMedia", new CORE_IMAGE_BRUSH_SVG("Sequencer/PlaybackControls/PlayControlsToFront", Icon20x20));
	Set("MediaPlayerEditor.JumpToEndMedia", new CORE_IMAGE_BRUSH_SVG("Sequencer/PlaybackControls/PlayControlsToEnd", Icon20x20));
	Set("MediaPlayerEditor.StepForwardMedia", new CORE_IMAGE_BRUSH_SVG("Sequencer/PlaybackControls/PlayControlsToNext", Icon20x20));
	Set("MediaPlayerEditor.StepBackwardMedia", new CORE_IMAGE_BRUSH_SVG("Sequencer/PlaybackControls/PlayControlsToPrevious", Icon20x20));	
	Set("MediaPlayerEditor.CloseMedia", new CORE_IMAGE_BRUSH_SVG("Starship/MainToolbar/eject", Icon20x20));
	Set("MediaPlayerEditor.OpenMedia", new IMAGE_BRUSH_SVG("icon_open_20x", Icon20x20));

	// scrubber
	Set("MediaPlayerEditor.Scrubber", FSliderStyle()
		.SetNormalBarImage(FSlateColorBrush(FColor::White))
		.SetDisabledBarImage(FSlateColorBrush(FLinearColor::Gray))
		.SetNormalThumbImage(IMAGE_BRUSH("scrubber", FVector2D(2.0f, 10.0f)))
		.SetHoveredThumbImage(IMAGE_BRUSH("scrubber", FVector2D(2.0f, 10.0f)))
		.SetDisabledThumbImage(IMAGE_BRUSH("scrubber", FVector2D(2.0f, 10.0f)))
		.SetBarThickness(2.0f)
	);

	Set("MediaPlayerEditor.ViewportFont", DEFAULT_FONT("Regular", 18));

	FSlateStyleRegistry::RegisterSlateStyle(*this);
}


FMediaPlayerEditorStyle::~FMediaPlayerEditorStyle()
{
	FSlateStyleRegistry::UnRegisterSlateStyle(*this);
}
