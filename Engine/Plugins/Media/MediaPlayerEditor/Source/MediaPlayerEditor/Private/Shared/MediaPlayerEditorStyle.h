// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Styling/SlateStyle.h"


/**
 * Implements the visual style of the media player editor UI.
 */
class FMediaPlayerEditorStyle
	: public FSlateStyleSet
{
public:

	/** Default constructor. */
	FMediaPlayerEditorStyle();

	 /** Destructor. */
	~FMediaPlayerEditorStyle();
};
