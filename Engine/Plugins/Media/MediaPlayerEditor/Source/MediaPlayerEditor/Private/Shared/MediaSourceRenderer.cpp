// Copyright Epic Games, Inc. All Rights Reserved.

#include "MediaSourceRenderer.h"

#include "MediaPlayer.h"
#include "MediaSource.h"
#include "MediaTexture.h"
#include "UObject/Package.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MediaSourceRenderer)

UMediaTexture* UMediaSourceRenderer::Open(UMediaSource* InMediaSource)
{
	if ((InMediaSource != nullptr) && (InMediaSource->Validate()))
	{
		// Set up the player.
		if (MediaPlayer == nullptr)
		{
			MediaPlayer = NewObject<UMediaPlayer>(GetTransientPackage());
			MediaPlayer->OnMediaOpened.AddDynamic(this, &UMediaSourceRenderer::OnMediaOpened);
			MediaPlayer->OnMediaOpenFailed.AddDynamic(this, &UMediaSourceRenderer::OnMediaOpenFailed);
		}
		else
		{
			MediaPlayer->Close();
		}

		// Set up the texture.
		if (MediaTexture == nullptr)
		{
			MediaTexture = NewObject<UMediaTexture>(GetTransientPackage());
			MediaTexture->NewStyleOutput = true;
		}
		MediaTexture->CurrentAspectRatio = 0.0f;
		MediaTexture->SetMediaPlayer(MediaPlayer.Get());
		MediaTexture->UpdateResource();
		MediaSource = InMediaSource;

		// Start playing the media.
		CurrentState = EState::Opening;
		WatchdogTimeRemaining = 10.0f;

		FMediaPlayerOptions Options;
		Options.PlayOnOpen = EMediaPlayerOptionBooleanOverride::Disabled;
		Options.Loop = EMediaPlayerOptionBooleanOverride::Disabled;
		// Let the media start at whichever time it defaults to.
		Options.SeekTimeType = EMediaPlayerOptionSeekTimeType::Ignored;
		// We don't need audio.
		Options.Tracks.Audio = -1;
		// For image media, we avoid filling the global cache which will needlessly hold onto frame data.
		Options.InternalCustomOptions.Emplace(MediaPlayerOptionValues::ImgMediaSmartCacheEnabled(), FVariant(true));
		Options.InternalCustomOptions.Emplace(MediaPlayerOptionValues::ImgMediaSmartCacheTimeToLookAhead(), FVariant(0.2f));
		bool bIsPlaying = MediaPlayer->OpenSourceWithOptions(MediaSource, Options);
		if (bIsPlaying == false)
		{
			Close();
		}
	}

	return MediaTexture.Get();
}

void UMediaSourceRenderer::Tick(float DeltaTime)
{
	if (MediaPlayer != nullptr)
	{
		// Keep the delta time in check. We typically create a thumbnail after having selected a new source
		// and if that brought up the system file selector box the time spent in the file selector is included
		// in the delta time.
		DeltaTime = FMath::Min(DeltaTime, 0.1f);

		// Is the texture ready?
		// The aspect ratio will change when we have something.
		if (MediaTexture->CurrentAspectRatio != 0.0f)
		{
			// Send this event so the content browser can update our thumbnail.
			if (MediaSource != nullptr)
			{
				FPropertyChangedEvent EmptyPropertyChangedEvent(nullptr);
				FCoreUObjectDelegates::OnObjectPropertyChanged.Broadcast(MediaSource,
					EmptyPropertyChangedEvent);
			}
			Close();
		}
		else
		{
			switch(CurrentState)
			{
				case EState::Closed:
				case EState::Errored:
				{
					break;
				}
				case EState::Opening:
				{
					// Make sure this doesn't drag on forever.
					if ((WatchdogTimeRemaining -= DeltaTime) < 0.0f)
					{
						CurrentState = EState::TimedOut;
					}
					break;
				}
				case EState::Open:
				{
					// Make sure this doesn't drag on forever.
					if ((WatchdogTimeRemaining -= DeltaTime) < 0.0f)
					{
						CurrentState = EState::TimedOut;
						return;
					}

					// Check that we have a media duration. When we have that we can assume
					// that we also know which tracks are present.
					const FTimespan MediaDuration = MediaPlayer->GetDuration();
					if (MediaDuration > FTimespan::Zero() && MediaDuration < FTimespan::MaxValue())
					{
						// Check if there is a video track.
						if (MediaPlayer->GetNumTracks(EMediaPlayerTrack::Video) <= 0 || MediaPlayer->GetTrackFormat(EMediaPlayerTrack::Video, 0) < 0)
						{
							CurrentState = EState::NotSupported;
						}
						else
						{
							// Check if the media can be seeked as we don't want the first frame
							// which is often just a black frame.
							if (MediaPlayer->SupportsSeeking())
							{
								const float kSeekPosAsDurationScale = 0.3f;

								FTimespan SeekTime = MediaDuration * kSeekPosAsDurationScale;
								// If seeking is supported, check if timeranges are as well.
								// If they are, chances are the media does not start at zero.
								if (MediaPlayer->SupportsPlaybackTimeRange())
								{
									TRange<FTimespan> Range = MediaPlayer->GetPlaybackTimeRange(EMediaTimeRangeType::Current);
									SeekTime = Range.GetLowerBoundValue();
									if (Range.GetUpperBoundValue() < FTimespan::MaxValue())
									{
										SeekTime += (Range.GetUpperBoundValue() - Range.GetLowerBoundValue()) * kSeekPosAsDurationScale;
									}
								}
								MediaPlayer->Seek(SeekTime);
							}
							MediaPlayer->Play();
							CurrentState = EState::Playing;
							WatchdogTimeRemaining = 5.0f;
						}
					}
					break;
				}
				case EState::Playing:
				{
					if ((WatchdogTimeRemaining -= DeltaTime) < 0.0f)
					{
						CurrentState = EState::TimedOut;
					}
					break;
				}
				case EState::TimedOut:
				case EState::NotSupported:
				case EState::Failed:
				{
					// Opening failed. No point in doing anything, so close everything.
					CurrentState = EState::Errored;
					Close();
					return;
				}
			}
		}
	}
}

void UMediaSourceRenderer::OnMediaOpened(FString)
{
	// Give it a moment in case the metadata is not immediately available.
	WatchdogTimeRemaining = 1.0f;
	CurrentState = EState::Open;
}

void UMediaSourceRenderer::OnMediaOpenFailed(FString)
{
	CurrentState = EState::Failed;
}

void UMediaSourceRenderer::Close()
{
	if (MediaTexture != nullptr)
	{
		MediaTexture->SetMediaPlayer(nullptr);
	}
	if (MediaPlayer != nullptr)
	{
		MediaPlayer->Close();
		MediaPlayer = nullptr;
	}
	CurrentState = EState::Closed;
	WatchdogTimeRemaining = 0.0f;
}
