// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MediaSourceRendererInterface.h"
#include "TickableEditorObject.h"
#include "UObject/Object.h"

#include "MediaSourceRenderer.generated.h"

class UMediaPlayer;
class UMediaSource;
class UMediaTexture;

/** Renders a media source to a texture in editor builds. */
UCLASS()
class UMediaSourceRenderer : public UObject
	, public IMediaSourceRendererInterface
	, public FTickableEditorObject
{
	GENERATED_BODY()

public:
	/** IMediaSourceRendererInterface interface */
	virtual UMediaTexture* Open(UMediaSource* InMediaSource) override;

	/** FTickableEditorObject interface */
	virtual void Tick(float DeltaTime) override;
	virtual TStatId GetStatId() const override { RETURN_QUICK_DECLARE_CYCLE_STAT(UMediaSourceRenderer, STATGROUP_Tickables); }

private:
	/**
	 * Cleans everything up.
	 * The media textue will remain so it can be used/reused.
	 */
	void Close();

	/** Holds the player we are using. */
	UPROPERTY(Transient)
	TObjectPtr<UMediaPlayer> MediaPlayer = nullptr;

	/** Holds the media source we are using. */
	UPROPERTY(Transient)
	TObjectPtr<UMediaSource> MediaSource = nullptr;

	/** Holds the media texture we are using. */
	UPROPERTY(Transient)
	TObjectPtr<UMediaTexture> MediaTexture = nullptr;

	enum class EState
	{
		Closed,
		Opening,
		Open,
		Playing,
		NotSupported,
		Failed,
		TimedOut,
		Errored
	};
	EState CurrentState = EState::Closed;
	float WatchdogTimeRemaining = 0.0f;
	UFUNCTION()
	void OnMediaOpened(FString InURL);
	UFUNCTION()
	void OnMediaOpenFailed(FString InURL);
};
