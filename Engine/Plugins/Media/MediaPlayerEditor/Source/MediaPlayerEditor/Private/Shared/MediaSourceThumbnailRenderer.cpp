// Copyright Epic Games, Inc. All Rights Reserved.

#include "MediaSourceThumbnailRenderer.h"
#include "CanvasTypes.h"
#include "MediaSource.h"
#include "MediaTexture.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MediaSourceThumbnailRenderer)

bool UMediaSourceThumbnailRenderer::CanVisualizeAsset(UObject* Object)
{
	// If the object has a new thumbnail then use that.
	return GetThumbnailTextureFromObject(Object) != nullptr;
}

void UMediaSourceThumbnailRenderer::GetThumbnailSize(UObject* Object, float Zoom, uint32& OutWidth, uint32& OutHeight) const
{
	UTexture* ObjectTexture = GetThumbnailTextureFromObject(Object);
	if (ObjectTexture != nullptr)
	{
		OutWidth = static_cast<uint32>(Zoom * ObjectTexture->GetSurfaceWidth());
		OutHeight = static_cast<uint32>(Zoom * ObjectTexture->GetSurfaceHeight());
	}
	else
	{
		OutWidth = 0;
		OutHeight = 0;
	}
}

void UMediaSourceThumbnailRenderer::Draw(UObject* Object, int32 X, int32 Y, uint32 Width, uint32 Height, FRenderTarget* RenderTarget, FCanvas* Canvas, bool bAdditionalViewFamily)
{
	UTexture* ObjectTexture = GetThumbnailTextureFromObject(Object);
	if (ObjectTexture != nullptr)
	{
		Super::Draw(ObjectTexture, X, Y, Width, Height, RenderTarget, Canvas, bAdditionalViewFamily);
	}
}

UTexture* UMediaSourceThumbnailRenderer::GetThumbnailTextureFromObject(UObject* Object) const
{
	UMediaSource* MediaSource = Cast<UMediaSource>(Object);
	if (MediaSource != nullptr)
	{
		UTexture* Texture = MediaSource->GetThumbnail();
		return Texture;
	}
	
	return nullptr;
}
