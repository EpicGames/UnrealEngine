// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ThumbnailRendering/TextureThumbnailRenderer.h"

#include "MediaSourceThumbnailRenderer.generated.h"

class UTexture;

/**
 * Renders thumbnails for media sources.
 */
UCLASS()
class UMediaSourceThumbnailRenderer : public UTextureThumbnailRenderer
{
	GENERATED_BODY()

public:

	// UThumbnailRenderer interface.
	virtual bool CanVisualizeAsset(UObject* Object) override;
	virtual void GetThumbnailSize(UObject* Object, float Zoom, uint32& OutWidth, uint32& OutHeight) const override;
	virtual void Draw(UObject* Object, int32 X, int32 Y, uint32 Width, uint32 Height, FRenderTarget*, FCanvas* Canvas, bool bAdditionalViewFamily) override;

protected:

	/**
	 * Gets the thumnbail from the object if the object is a media source.
	 */
	virtual UTexture* GetThumbnailTextureFromObject(UObject* Object) const;

};
