// Copyright Epic Games, Inc. All Rights Reserved.

#include "Toolkits/MediaPlayerEditorToolkit.h"

#include "Framework/Docking/TabManager.h"
#include "Framework/MultiBox/MultiBoxExtender.h"
#include "MediaPlayer.h"
#include "Models/MediaPlayerEditorCommands.h"
#include "SlateOptMacros.h"
#include "Widgets/Docking/SDockTab.h"
#include "Widgets/SMediaImageTextureChannelToggle.h"
#include "Widgets/SMediaPlayerEditorDetails.h"
#include "Widgets/SMediaPlayerEditorInfo.h"
#include "Widgets/SMediaPlayerEditorMedia.h"
#include "Widgets/SMediaPlayerEditorPlaylist.h"
#include "Widgets/SMediaPlayerEditorStats.h"
#include "Widgets/SMediaPlayerEditorViewer.h"

#define LOCTEXT_NAMESPACE "FMediaPlayerEditorToolkit"

namespace MediaPlayerEditorToolkit
{
	static const FName AppIdentifier("MediaPlayerEditorApp");
	static const FName DetailsTabId("Details");
	static const FName InfoTabId("Info");
	static const FName MediaTabId("Media");
	static const FName PlaylistTabId("Playlist");
	static const FName StatsTabId("Stats");
	static const FName ViewerTabId("Viewer");
}

FMediaPlayerEditorToolkit::FMediaPlayerEditorToolkit(const TSharedRef<ISlateStyle>& InStyle)
	: FMediaPlayerEditorToolkitMediaPlayerBase(InStyle)
{}

void FMediaPlayerEditorToolkit::Initialize(UMediaPlayer* InMediaPlayer, const EToolkitMode::Type InMode, const TSharedPtr<IToolkitHost>& InToolkitHost)
{
	MediaPlayer = InMediaPlayer;

	if (MediaPlayer == nullptr)
	{
		return;
	}

	// support undo/redo
	MediaPlayer->SetFlags(RF_Transactional);

	FMediaPlayerEditorToolkitMediaPlayerBase::Initialize(InMediaPlayer, MediaPlayerEditorToolkit::AppIdentifier, InMode, InToolkitHost);
	
//	IMediaPlayerEditorModule* MediaPlayerEditorModule = &FModuleManager::LoadModuleChecked<IMediaPlayerEditorModule>("MediaPlayerEditor");
//	AddMenuExtender(MediaPlayerEditorModule->GetMenuExtensibilityManager()->GetAllExtenders(GetToolkitCommands(), GetEditingObjects()));
}

TSharedRef<FTabManager::FLayout> FMediaPlayerEditorToolkit::CreateLayout()
{
	return FTabManager::NewLayout("Standalone_MediaPlayerEditor_v11")
		->AddArea
		(
			FTabManager::NewPrimaryArea()
				->SetOrientation(Orient_Horizontal)
				->Split
				(
					FTabManager::NewSplitter()
						->SetOrientation(Orient_Vertical)
						->Split
						(
							// viewer
							FTabManager::NewStack()
								->AddTab(MediaPlayerEditorToolkit::ViewerTabId, ETabState::OpenedTab)
								->SetHideTabWell(true)
								->SetSizeCoefficient(0.6f)
						)
						->Split
						(
							// media library
							FTabManager::NewStack()
								->AddTab(MediaPlayerEditorToolkit::MediaTabId, ETabState::OpenedTab)
								->SetHideTabWell(true)
								->SetSizeCoefficient(0.3f)
						)
				)
				->Split
				(
					FTabManager::NewSplitter()
						->SetOrientation(Orient_Vertical)
						->SetSizeCoefficient(0.33f)
						->Split
						(
							// playlist
							FTabManager::NewStack()
								->AddTab(MediaPlayerEditorToolkit::PlaylistTabId, ETabState::OpenedTab)
								->SetSizeCoefficient(0.5f)
						)
						->Split
						(
							// details, info, stats
							FTabManager::NewStack()
								->AddTab(MediaPlayerEditorToolkit::DetailsTabId, ETabState::OpenedTab)
								->AddTab(MediaPlayerEditorToolkit::InfoTabId, ETabState::OpenedTab)
								->AddTab(MediaPlayerEditorToolkit::StatsTabId, ETabState::ClosedTab)
								->SetForegroundTab(MediaPlayerEditorToolkit::DetailsTabId)
								->SetSizeCoefficient(0.5f)
						)
				)
		);
}

void FMediaPlayerEditorToolkit::OnClose()
{
	if (MediaPlayer != nullptr)
	{
		MediaPlayer->Close();
	}
}

void FMediaPlayerEditorToolkit::RegisterTabSpawners(const TSharedRef<FTabManager>& InTabManager)
{
	WorkspaceMenuCategory = InTabManager->AddLocalWorkspaceMenuCategory(LOCTEXT("WorkspaceMenu_MediaPlayerEditor", "Media Player Editor"));
	auto WorkspaceMenuCategoryRef = WorkspaceMenuCategory.ToSharedRef();

	FAssetEditorToolkit::RegisterTabSpawners(InTabManager);

	InTabManager->RegisterTabSpawner(MediaPlayerEditorToolkit::DetailsTabId, FOnSpawnTab::CreateSP(this, &FMediaPlayerEditorToolkit::HandleTabManagerSpawnTab, MediaPlayerEditorToolkit::DetailsTabId))
		.SetDisplayName(LOCTEXT("DetailsTabName", "Details"))
		.SetGroup(WorkspaceMenuCategoryRef)
		.SetIcon(FSlateIcon(FAppStyle::GetAppStyleSetName(), "LevelEditor.Tabs.Details"));

	InTabManager->RegisterTabSpawner(MediaPlayerEditorToolkit::InfoTabId, FOnSpawnTab::CreateSP(this, &FMediaPlayerEditorToolkit::HandleTabManagerSpawnTab, MediaPlayerEditorToolkit::InfoTabId))
		.SetDisplayName(LOCTEXT("InfoTabName", "Info"))
		.SetGroup(WorkspaceMenuCategoryRef)
		.SetIcon(FSlateIcon(Style->GetStyleSetName(), "MediaPlayerEditor.Tabs.Info"));

	InTabManager->RegisterTabSpawner(MediaPlayerEditorToolkit::MediaTabId, FOnSpawnTab::CreateSP(this, &FMediaPlayerEditorToolkit::HandleTabManagerSpawnTab, MediaPlayerEditorToolkit::MediaTabId))
		.SetDisplayName(LOCTEXT("MediaTabName", "Media Library"))
		.SetGroup(WorkspaceMenuCategoryRef)
		.SetIcon(FSlateIcon(Style->GetStyleSetName(), "MediaPlayerEditor.Tabs.Media"));

	InTabManager->RegisterTabSpawner(MediaPlayerEditorToolkit::ViewerTabId, FOnSpawnTab::CreateSP(this, &FMediaPlayerEditorToolkit::HandleTabManagerSpawnTab, MediaPlayerEditorToolkit::ViewerTabId))
		.SetDisplayName(LOCTEXT("PlayerTabName", "Player"))
		.SetGroup(WorkspaceMenuCategoryRef)
		.SetIcon(FSlateIcon(Style->GetStyleSetName(), "MediaPlayerEditor.Tabs.Player"));

	InTabManager->RegisterTabSpawner(MediaPlayerEditorToolkit::PlaylistTabId, FOnSpawnTab::CreateSP(this, &FMediaPlayerEditorToolkit::HandleTabManagerSpawnTab, MediaPlayerEditorToolkit::PlaylistTabId))
		.SetDisplayName(LOCTEXT("PlaylistTabName", "Playlist"))
		.SetGroup(WorkspaceMenuCategoryRef)
		.SetIcon(FSlateIcon(Style->GetStyleSetName(), "MediaPlayerEditor.Tabs.Playlist"));

	InTabManager->RegisterTabSpawner(MediaPlayerEditorToolkit::StatsTabId, FOnSpawnTab::CreateSP(this, &FMediaPlayerEditorToolkit::HandleTabManagerSpawnTab, MediaPlayerEditorToolkit::StatsTabId))
		.SetDisplayName(LOCTEXT("StatsTabName", "Stats"))
		.SetGroup(WorkspaceMenuCategoryRef)
		.SetIcon(FSlateIcon(Style->GetStyleSetName(), "MediaPlayerEditor.Tabs.Stats"));
}

void FMediaPlayerEditorToolkit::UnregisterTabSpawners(const TSharedRef<class FTabManager>& InTabManager)
{
	FAssetEditorToolkit::UnregisterTabSpawners(InTabManager);

	InTabManager->UnregisterTabSpawner(MediaPlayerEditorToolkit::ViewerTabId);
	InTabManager->UnregisterTabSpawner(MediaPlayerEditorToolkit::StatsTabId);
	InTabManager->UnregisterTabSpawner(MediaPlayerEditorToolkit::PlaylistTabId);
	InTabManager->UnregisterTabSpawner(MediaPlayerEditorToolkit::MediaTabId);
	InTabManager->UnregisterTabSpawner(MediaPlayerEditorToolkit::InfoTabId);
	InTabManager->UnregisterTabSpawner(MediaPlayerEditorToolkit::DetailsTabId);
}

FText FMediaPlayerEditorToolkit::GetBaseToolkitName() const
{
	return LOCTEXT("AppLabel", "Media Player Editor");
}

FName FMediaPlayerEditorToolkit::GetToolkitFName() const
{
	return FName("MediaPlayerEditor");
}

FString FMediaPlayerEditorToolkit::GetWorldCentricTabPrefix() const
{
	return LOCTEXT("WorldCentricTabPrefix", "MediaPlayer ").ToString();
}

BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION
void FMediaPlayerEditorToolkit::ExtendToolBar()
{
	TSharedPtr<FExtender> ToolbarExtender = MakeShared<FExtender>();

	ToolbarExtender->AddToolBarExtension(
		"Asset",
		EExtensionHook::After,
		GetToolkitCommands(),
		FToolBarExtensionDelegate::CreateLambda(
			[this](FToolBarBuilder& ToolbarBuilder)
			{
				using namespace MediaPlayerEditor::MediaImage;

				ToolbarBuilder.BeginSection("TextureControls");
				{
					ToolbarBuilder.AddWidget(SNew(SMediaImageTextureChannelToggle, Viewer, ETextureChannelMask::Red, ToolkitCommands));
					ToolbarBuilder.AddWidget(SNew(SMediaImageTextureChannelToggle, Viewer, ETextureChannelMask::Green, ToolkitCommands));
					ToolbarBuilder.AddWidget(SNew(SMediaImageTextureChannelToggle, Viewer, ETextureChannelMask::Blue, ToolkitCommands));
					ToolbarBuilder.AddWidget(SNew(SMediaImageTextureChannelToggle, Viewer, ETextureChannelMask::Alpha, ToolkitCommands));
				}
				ToolbarBuilder.EndSection();
			}
		)
	);

	AddToolbarExtender(ToolbarExtender);
}
END_SLATE_FUNCTION_BUILD_OPTIMIZATION

TSharedRef<SDockTab> FMediaPlayerEditorToolkit::HandleTabManagerSpawnTab(const FSpawnTabArgs& Args, FName TabIdentifier)
{
	TSharedPtr<SWidget> TabWidget = SNullWidget::NullWidget;

	if (TabIdentifier == MediaPlayerEditorToolkit::DetailsTabId)
	{
		TabWidget = SNew(SMediaPlayerEditorDetails, *MediaPlayer, Style);
	}
	else if (TabIdentifier == MediaPlayerEditorToolkit::InfoTabId)
	{
		TabWidget = SNew(SMediaPlayerEditorInfo, *MediaPlayer, Style);
	}
	else if (TabIdentifier == MediaPlayerEditorToolkit::MediaTabId)
	{
		TabWidget = SNew(SMediaPlayerEditorMedia, *MediaPlayer, Style);
	}
	else if (TabIdentifier == MediaPlayerEditorToolkit::PlaylistTabId)
	{
		TabWidget = SNew(SMediaPlayerEditorPlaylist, *MediaPlayer, Style);
	}
	else if (TabIdentifier == MediaPlayerEditorToolkit::StatsTabId)
	{
		TabWidget = SNew(SMediaPlayerEditorStats, *MediaPlayer, Style);
	}
	else if (TabIdentifier == MediaPlayerEditorToolkit::ViewerTabId)
	{
		TabWidget = SAssignNew(Viewer, SMediaPlayerEditorViewer, *MediaPlayer, nullptr, Style, true)
			.Commands(ToolkitCommands);
	}

	return SNew(SDockTab)
		.TabRole(ETabRole::PanelTab)
		[
			TabWidget.ToSharedRef()
		];
}

#undef LOCTEXT_NAMESPACE
