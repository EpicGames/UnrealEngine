// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Toolkits/MediaPlayerEditorToolkitMediaPlayerBase.h"

class FSpawnTabArgs;
class SDockTab;
class UMediaPlayer;

/**
 * Implements an Editor toolkit for media players.
 */
class FMediaPlayerEditorToolkit
	: public FMediaPlayerEditorToolkitMediaPlayerBase
{
public:
	/**
	 * Creates and initializes a new instance.
	 *
	 * @param InStyle The style set to use.
	 */
	FMediaPlayerEditorToolkit(const TSharedRef<ISlateStyle>& InStyle);

	/**
	 * Initializes the editor tool kit.
	 *
	 * @param InMediaPlayer The UMediaPlayer asset to edit.
	 * @param InMode The mode to create the toolkit in.
	 * @param InToolkitHost The toolkit host.
	 */
	void Initialize(UMediaPlayer* InMediaPlayer, const EToolkitMode::Type InMode, const TSharedPtr<IToolkitHost>& InToolkitHost);

	//~ Begin FAssetEditorToolkit
	virtual void OnClose() override;
	virtual void RegisterTabSpawners(const TSharedRef<FTabManager>& TabManager) override;
	virtual void UnregisterTabSpawners(const TSharedRef<FTabManager>& TabManager) override;
	//~ End FAssetEditorToolkit

	//~ Begin IToolkit
	virtual FText GetBaseToolkitName() const override;
	virtual FName GetToolkitFName() const override;
	virtual FString GetWorldCentricTabPrefix() const override;
	//~ End IToolkit

	//~ Begin FGCObject
	virtual FString GetReferencerName() const override
	{
		return TEXT("FMediaPlayerEditorToolkit");
	}
	//~ End FGCObject

protected:
	//~ Begin FMediaPlayerToolkitBase
	virtual TSharedRef<FTabManager::FLayout> CreateLayout() override;
	virtual void ExtendToolBar() override;
	//~ End FMediaPlayerToolkitBase

private:
	/** Callback for spawning tabs. */
	TSharedRef<SDockTab> HandleTabManagerSpawnTab(const FSpawnTabArgs& Args, FName TabIdentifier);
};
