// Copyright Epic Games, Inc. All Rights Reserved.

#include "Toolkits/MediaPlayerEditorToolkitBase.h"

#include "Editor.h"
#include "EditorReimportHandler.h"

FMediaPlayerEditorToolkitBase::FMediaPlayerEditorToolkitBase(const TSharedRef<ISlateStyle>& InStyle)
	: Style(InStyle)
{
}

FMediaPlayerEditorToolkitBase::~FMediaPlayerEditorToolkitBase()
{
	FReimportManager::Instance()->OnPreReimport().RemoveAll(this);
	FReimportManager::Instance()->OnPostReimport().RemoveAll(this);

	if (GEditor)
	{
		GEditor->UnregisterForUndo(this);
	}
}

FString FMediaPlayerEditorToolkitBase::GetDocumentationLink() const
{
	return FString(TEXT("WorkingWithMedia/IntegratingMedia/MediaFramework"));
}

FLinearColor FMediaPlayerEditorToolkitBase::GetWorldCentricTabColorScale() const
{
	return FLinearColor(0.3f, 0.2f, 0.5f, 0.5f);
}

void FMediaPlayerEditorToolkitBase::PostUndo(bool bSuccess)
{
	// Do nothing
}

void FMediaPlayerEditorToolkitBase::PostRedo(bool bSuccess)
{
	PostUndo(bSuccess);
}

void FMediaPlayerEditorToolkitBase::Initialize(UObject* InAsset, FName InAppIdentifier, const EToolkitMode::Type InMode, const TSharedPtr<IToolkitHost>& InToolkitHost)
{
	GEditor->RegisterForUndo(this);

	BindCommands();

	const TSharedRef<FTabManager::FLayout> Layout = CreateLayout();

	FAssetEditorToolkit::InitAssetEditor(
		InMode,
		InToolkitHost,
		InAppIdentifier,
		Layout,
		/* bCreateDefaultStandaloneMenu */ true,
		/* bCreateDefaultToolbar */ true,
		InAsset
	);

	ExtendToolBar();
	RegenerateMenusAndToolbars();
}
