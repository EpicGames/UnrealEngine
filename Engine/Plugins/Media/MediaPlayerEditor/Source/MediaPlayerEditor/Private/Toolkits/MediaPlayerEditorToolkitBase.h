// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "EditorUndoClient.h"
#include "Toolkits/AssetEditorToolkit.h"
#include "UObject/GCObject.h"

class ISlateStyle;

/**
 * Base class for all Media Player Editor Toolkits
 */
class FMediaPlayerEditorToolkitBase
	: public FAssetEditorToolkit
	, public FEditorUndoClient
	, public FGCObject
{
public:
	/**
	 * Creates and initializes a new instance.
	 *
	 * @param InStyle The style set to use.
	 */
	FMediaPlayerEditorToolkitBase(const TSharedRef<ISlateStyle>& InStyle);

	/** Virtual destructor. */
	virtual ~FMediaPlayerEditorToolkitBase();

	//~ Begin FAssetEditorToolkit
	virtual FString GetDocumentationLink() const override;
	//~ End FAssetEditorToolkit

	//~ Begin IToolkit
	virtual FLinearColor GetWorldCentricTabColorScale() const override;
	//~ End IToolkit

	//~ Begin FEditorUndoClient
	virtual void PostUndo(bool bSuccess) override;
	virtual void PostRedo(bool bSuccess) override;
	//~ End FEditorUndoClient

protected:
	/** Pointer to the style set to use for toolkits. */
	TSharedRef<ISlateStyle> Style;

	/** Binds the UI commands to delegates. */
	virtual void BindCommands() {}

	/** Sets up the default layout for the toolkit */
	virtual TSharedRef<FTabManager::FLayout> CreateLayout() = 0;

	/** Builds the toolbar widget for the media player editor. */
	virtual void ExtendToolBar() {}

	/**
	 * Initializes the editor tool kit.
	 *
	 * @param InAsset The object this toolkit is editing.
	 * @param InAppIdentifier The identifier for this specific toolkit.
	 * @param InMode The mode to create the toolkit in.
	 * @param InToolkitHost The toolkit host.
	 */
	void Initialize(UObject* InAsset, FName InAppIdentifier, const EToolkitMode::Type InMode, const TSharedPtr<IToolkitHost>& InToolkitHost);
};
