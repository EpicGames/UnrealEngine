// Copyright Epic Games, Inc. All Rights Reserved.

#include "Toolkits/MediaPlayerEditorToolkitMediaPlayerBase.h"

#include "Framework/Commands/UICommandList.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "MediaPlayer.h"
#include "MediaPlaylist.h"
#include "Models/MediaPlayerEditorCommands.h"
#include "UObject/UObjectGlobals.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/SMediaPlayerEditorViewer.h"

FMediaPlayerEditorToolkitMediaPlayerBase::FMediaPlayerEditorToolkitMediaPlayerBase(const TSharedRef<ISlateStyle>& InStyle)
	: FMediaPlayerEditorToolkitBase(InStyle)
	, MediaPlayer(nullptr)
{
}

MediaPlayerEditor::MediaImage::ETextureChannelMask FMediaPlayerEditorToolkitMediaPlayerBase::GetChannelMask() const
{
	if (Viewer.IsValid())
	{
		return Viewer->GetChannelMask();
	}

	return MediaPlayerEditor::MediaImage::ETextureChannelMask::RGBA;
}

bool FMediaPlayerEditorToolkitMediaPlayerBase::IsChannelMasked(MediaPlayerEditor::MediaImage::ETextureChannelMask InChannel) const
{
	if (Viewer.IsValid())
	{
		return Viewer->IsChannelMasked(InChannel);
	}

	return true;
}

void FMediaPlayerEditorToolkitMediaPlayerBase::ToggleChannelMask(MediaPlayerEditor::MediaImage::ETextureChannelMask InChannelToToggle)
{
	if (Viewer.IsValid())
	{
		Viewer->ToggleChannelMask(InChannelToToggle);
	}
}

void FMediaPlayerEditorToolkitMediaPlayerBase::SetChannelMask(MediaPlayerEditor::MediaImage::ETextureChannelMask InMask)
{
	if (Viewer.IsValid())
	{
		Viewer->SetChannelMask(InMask);
	}
}

void FMediaPlayerEditorToolkitMediaPlayerBase::OnClose()
{
	if (MediaPlayer != nullptr)
	{
		MediaPlayer->Close();
	}
}

void FMediaPlayerEditorToolkitMediaPlayerBase::AddReferencedObjects(FReferenceCollector& InCollector)
{
	InCollector.AddReferencedObject(MediaPlayer);
}

float FMediaPlayerEditorToolkitMediaPlayerBase::GetForwardRate() const
{
	float Rate = MediaPlayer->GetRate();

	if (Rate < 1.0f)
	{
		Rate = 1.0f;
	}

	return 2 * Rate;
}

float FMediaPlayerEditorToolkitMediaPlayerBase::GetReverseRate() const
{
	float Rate = MediaPlayer->GetRate();

	if (Rate > -1.0f)
	{
		return -1.0f;
	}

	return 2 * Rate;
}

void FMediaPlayerEditorToolkitMediaPlayerBase::BindCommands()
{
	const FMediaPlayerEditorCommands& Commands = FMediaPlayerEditorCommands::Get();

	ToolkitCommands->MapAction(
		Commands.CloseMedia,
		FExecuteAction::CreateLambda([this] { MediaPlayer->Close(); }),
		FCanExecuteAction::CreateLambda([this] { return !MediaPlayer->GetUrl().IsEmpty(); })
	);

	ToolkitCommands->MapAction(
		Commands.ForwardMedia,
		FExecuteAction::CreateLambda([this] { MediaPlayer->SetRate(GetForwardRate()); }),
		FCanExecuteAction::CreateLambda([this] { return MediaPlayer->IsReady() && MediaPlayer->SupportsRate(GetForwardRate(), false); })
	);

	ToolkitCommands->MapAction(
		Commands.PlayReverseMedia,
		FExecuteAction::CreateLambda([this] { MediaPlayer->SetRate(-1.f); }),
		FCanExecuteAction::CreateLambda([this] { return MediaPlayer->IsReady() && MediaPlayer->SupportsRate(-1.f,  /* Unthinned */ true) && (!MediaPlayer->IsPlaying() || (MediaPlayer->GetRate() != -1.0f)); })
	);

	ToolkitCommands->MapAction(
		Commands.NextMedia,
		FExecuteAction::CreateLambda([this]{ MediaPlayer->Next(); }),
		FCanExecuteAction::CreateLambda([this]{ return (MediaPlayer->GetPlaylistRef().Num() > 1); })
	);

	ToolkitCommands->MapAction(
		Commands.PauseMedia,
		FExecuteAction::CreateLambda([this]{ MediaPlayer->Pause(); }),
		FCanExecuteAction::CreateLambda([this]{ return MediaPlayer->CanPause() && !MediaPlayer->IsPaused(); })
	);

	ToolkitCommands->MapAction(
		Commands.PlayMedia,
		FExecuteAction::CreateLambda([this]{ MediaPlayer->Play(); }),
		FCanExecuteAction::CreateLambda([this]{ return MediaPlayer->IsReady() && (!MediaPlayer->IsPlaying() || (MediaPlayer->GetRate() != 1.0f)); })
	);

	ToolkitCommands->MapAction(
		Commands.TogglePlayPauseMedia,
		FExecuteAction::CreateLambda(
			[this]
			{
				if (MediaPlayer->IsPlaying())
				{
					MediaPlayer->Pause();
				}
				else
				{
					MediaPlayer->Play();
				}
			}),
		FCanExecuteAction::CreateLambda([this] { return MediaPlayer->IsReady() && (MediaPlayer->IsPlaying() || MediaPlayer->GetRate() != 1.f); })
	);

	ToolkitCommands->MapAction(
		Commands.TogglePlayReversePauseMedia,
		FExecuteAction::CreateLambda(
			[this]
			{
				if (MediaPlayer->IsPlaying())
				{
					MediaPlayer->Pause();
				}
				else
				{
					MediaPlayer->SetRate(-1.f);
				}
			}),
		FCanExecuteAction::CreateLambda([this] { return MediaPlayer->IsReady() && MediaPlayer->SupportsRate(-1.f, /* Unthinned */ true) && (MediaPlayer->IsPlaying() || MediaPlayer->GetRate() != -1.f ); })
	);

	ToolkitCommands->MapAction(
		Commands.PreviousMedia,
		FExecuteAction::CreateLambda([this]{ MediaPlayer->Previous(); }),
		FCanExecuteAction::CreateLambda([this]{ return (MediaPlayer->GetPlaylistRef().Num() > 1); })
	);

	ToolkitCommands->MapAction(
		Commands.ReverseMedia,
		FExecuteAction::CreateLambda([this] { MediaPlayer->SetRate(GetReverseRate()); }),
		FCanExecuteAction::CreateLambda([this] { return MediaPlayer->IsReady() && MediaPlayer->SupportsRate(GetReverseRate(), false); })
	);

	ToolkitCommands->MapAction(
		Commands.RewindMedia,
		FExecuteAction::CreateLambda([this]{ MediaPlayer->Rewind(); }),
		FCanExecuteAction::CreateLambda([this]{ return MediaPlayer->IsReady() && MediaPlayer->SupportsSeeking() && MediaPlayer->GetTime() > FTimespan::Zero(); })
	);

	ToolkitCommands->MapAction(
		Commands.JumpToEndMedia,
		FExecuteAction::CreateLambda([this] { MediaPlayer->Seek(MediaPlayer->GetDuration()); }),
		FCanExecuteAction::CreateLambda([this] { return MediaPlayer->IsReady() && MediaPlayer->SupportsSeeking(); })
	);

	static const FTimespan TenSeconds = FTimespan::FromSeconds(10.f);

	ToolkitCommands->MapAction(
		Commands.StepBackwardMedia,
		FExecuteAction::CreateLambda([this] 
			{
				// TODO: Convert to underlying media player command when framework support is added.
				const float FrameRate = MediaPlayer->GetVideoTrackFrameRate(INDEX_NONE, INDEX_NONE);

				if (!FMath::IsNearlyZero(FrameRate))
				{
					MediaPlayer->Seek(MediaPlayer->GetTime() - FTimespan::FromSeconds(1.f / FrameRate));
				}
			}
		),
		FCanExecuteAction::CreateLambda([this] { return MediaPlayer->IsReady() && MediaPlayer->IsPaused() && MediaPlayer->SupportsSeeking(); })
	);

	ToolkitCommands->MapAction(
		Commands.StepForwardMedia,
		FExecuteAction::CreateLambda([this] 
			{
				// TODO: Convert to underlying media player command when framework support is added.
				const float FrameRate = MediaPlayer->GetVideoTrackFrameRate(INDEX_NONE, INDEX_NONE);

				if (!FMath::IsNearlyZero(FrameRate))
				{
					MediaPlayer->Seek(MediaPlayer->GetTime() + FTimespan::FromSeconds(1.f / FrameRate));
				}
			}
		),
		FCanExecuteAction::CreateLambda([this] { return MediaPlayer->IsReady() && MediaPlayer->IsPaused() && MediaPlayer->SupportsSeeking(); })
	);

	ToolkitCommands->MapAction(
		Commands.ToggledRedTextureChannel,
		FExecuteAction::CreateLambda([this] { ToggleChannelMask(MediaPlayerEditor::MediaImage::ETextureChannelMask::Red); }),
		FCanExecuteAction::CreateLambda([this] { return Viewer.IsValid(); }),
		FIsActionChecked::CreateLambda([this] { return IsChannelMasked(MediaPlayerEditor::MediaImage::ETextureChannelMask::Red); })
	);

	ToolkitCommands->MapAction(
		Commands.ToggledGreenTextureChannel,
		FExecuteAction::CreateLambda([this] { ToggleChannelMask(MediaPlayerEditor::MediaImage::ETextureChannelMask::Green); }),
		FCanExecuteAction::CreateLambda([this] { return Viewer.IsValid(); }),
		FIsActionChecked::CreateLambda([this] { return IsChannelMasked(MediaPlayerEditor::MediaImage::ETextureChannelMask::Green); })
	);

	ToolkitCommands->MapAction(
		Commands.ToggledBlueTextureChannel,
		FExecuteAction::CreateLambda([this] { ToggleChannelMask(MediaPlayerEditor::MediaImage::ETextureChannelMask::Blue); }),
		FCanExecuteAction::CreateLambda([this] { return Viewer.IsValid(); }),
		FIsActionChecked::CreateLambda([this] { return IsChannelMasked(MediaPlayerEditor::MediaImage::ETextureChannelMask::Blue); })
	);

	ToolkitCommands->MapAction(
		Commands.ToggledAlphaTextureChannel,
		FExecuteAction::CreateLambda([this] { ToggleChannelMask(MediaPlayerEditor::MediaImage::ETextureChannelMask::Alpha); }),
		FCanExecuteAction::CreateLambda([this] { return Viewer.IsValid(); }),
		FIsActionChecked::CreateLambda([this] { return IsChannelMasked(MediaPlayerEditor::MediaImage::ETextureChannelMask::Alpha); })
	);
}
