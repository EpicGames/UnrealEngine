// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Toolkits/MediaPlayerEditorToolkitBase.h"

#include "Widgets/SMediaImage.h"

class FToolBarBuilder;
class SMediaPlayerEditorViewer;
class UMediaPlayer;

#define UE_API MEDIAPLAYEREDITOR_API

/**
 * Base class for all Media Player Editor Toolkits using Media Players
 */
class FMediaPlayerEditorToolkitMediaPlayerBase
	: public FMediaPlayerEditorToolkitBase
{
public:
	/**
	 * Creates and initializes a new instance.
	 *
	 * @param InStyle The style set to use.
	 */
	FMediaPlayerEditorToolkitMediaPlayerBase(const TSharedRef<ISlateStyle>& InStyle);

	/**
	 * Returns the active mask.
	 */
	MediaPlayerEditor::MediaImage::ETextureChannelMask GetChannelMask() const;

	/**
	 * Returns true if all the given channel mask is enabled.
	 */
	bool IsChannelMasked(MediaPlayerEditor::MediaImage::ETextureChannelMask InChannel) const;

	/**
	 * Sets a channel-based mask for the image.
	 *
	 * If only a single channel is displayed, it will show in greyscale.
	 *
	 * @param InMask to display.
	 */
	void SetChannelMask(MediaPlayerEditor::MediaImage::ETextureChannelMask InMask);

	/**
	 * Toggles the given mask in the channel mask.
	 */
	void ToggleChannelMask(MediaPlayerEditor::MediaImage::ETextureChannelMask InChannelToToggle);

	//~ Begin FAssetEditorToolkit
	virtual void OnClose() override;
	//~ End FAssetEditorToolkit

	//~ Begin FGCObject
	virtual void AddReferencedObjects(FReferenceCollector& InCollector) override;
	//~ End FGCObject

protected:
	/** The media player to play the media with. */
	TObjectPtr<UMediaPlayer> MediaPlayer;

	/** The tab which displays the media. */
	TSharedPtr<SMediaPlayerEditorViewer> Viewer;

	/**
	 * Gets the playback rate for fast forward.
	 *
	 * @return Forward playback rate.
	 */
	float GetForwardRate() const;

	/**
	 * Gets the playback rate for reverse.
	 *
	 * @return Reverse playback rate.
	 */
	float GetReverseRate() const;

	//~ Begin FMediaPlayerEditorToolkitBase
	virtual void BindCommands() override;
	//~ End FMediaPlayerEditorToolkitBase
};

#undef UE_API
