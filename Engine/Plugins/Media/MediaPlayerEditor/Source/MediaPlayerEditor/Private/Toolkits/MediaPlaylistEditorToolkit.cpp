// Copyright Epic Games, Inc. All Rights Reserved.

#include "Toolkits/MediaPlaylistEditorToolkit.h"

#include "Framework/Docking/TabManager.h"
#include "MediaPlaylist.h"
#include "Widgets/Docking/SDockTab.h"
#include "Widgets/SMediaPlaylistEditorDetails.h"
#include "Widgets/SMediaPlaylistEditorMedia.h"

#define LOCTEXT_NAMESPACE "FMediaPlaylistEditorToolkit"

namespace MediaPlaylistEditorToolkit
{
	static const FName AppIdentifier("MediaPlaylistEditorApp");
	static const FName DetailsTabId("Details");
	static const FName MediaTabId("Media");
}

FMediaPlaylistEditorToolkit::FMediaPlaylistEditorToolkit(const TSharedRef<ISlateStyle>& InStyle)
	: FMediaPlayerEditorToolkitBase(InStyle)
	, MediaPlaylist(nullptr)
{}

void FMediaPlaylistEditorToolkit::Initialize(UMediaPlaylist* InMediaPlaylist, const EToolkitMode::Type InMode, const TSharedPtr<IToolkitHost>& InToolkitHost)
{
	MediaPlaylist = InMediaPlaylist;

	if (MediaPlaylist == nullptr)
	{
		return;
	}

	// Support undo/redo
	MediaPlaylist->SetFlags(RF_Transactional);

	FMediaPlayerEditorToolkitBase::Initialize(
		InMediaPlaylist,
		MediaPlaylistEditorToolkit::AppIdentifier,
		InMode, 
		InToolkitHost
	);
}

TSharedRef<FTabManager::FLayout> FMediaPlaylistEditorToolkit::CreateLayout()
{
	return FTabManager::NewLayout("Standalone_MediaPlaylistEditor_v3")
		->AddArea
		(
			FTabManager::NewPrimaryArea()
				->SetOrientation(Orient_Vertical)
				->Split
				(
					// details
					FTabManager::NewStack()
						->AddTab(MediaPlaylistEditorToolkit::DetailsTabId, ETabState::OpenedTab)
						->SetSizeCoefficient(0.45f)
				)
				->Split
				(
					// media library
					FTabManager::NewStack()
						->AddTab(MediaPlaylistEditorToolkit::MediaTabId, ETabState::OpenedTab)
						->SetHideTabWell(true)
						->SetSizeCoefficient(0.45f)
				)
		);
}

void FMediaPlaylistEditorToolkit::RegisterTabSpawners(const TSharedRef<FTabManager>& InTabManager)
{
	WorkspaceMenuCategory = InTabManager->AddLocalWorkspaceMenuCategory(LOCTEXT("WorkspaceMenu_MediaPlaylistEditor", "Media Player Editor"));
	auto WorkspaceMenuCategoryRef = WorkspaceMenuCategory.ToSharedRef();

	FAssetEditorToolkit::RegisterTabSpawners(InTabManager);

	InTabManager->RegisterTabSpawner(MediaPlaylistEditorToolkit::DetailsTabId, FOnSpawnTab::CreateSP(this, &FMediaPlaylistEditorToolkit::HandleTabManagerSpawnTab, MediaPlaylistEditorToolkit::DetailsTabId))
		.SetDisplayName(LOCTEXT("DetailsTabName", "Details"))
		.SetGroup(WorkspaceMenuCategoryRef)
		.SetIcon(FSlateIcon(FAppStyle::GetAppStyleSetName(), "LevelEditor.Tabs.Details"));

	InTabManager->RegisterTabSpawner(MediaPlaylistEditorToolkit::MediaTabId, FOnSpawnTab::CreateSP(this, &FMediaPlaylistEditorToolkit::HandleTabManagerSpawnTab, MediaPlaylistEditorToolkit::MediaTabId))
		.SetDisplayName(LOCTEXT("MediaTabName", "Media Library"))
		.SetGroup(WorkspaceMenuCategoryRef)
		.SetIcon(FSlateIcon(Style->GetStyleSetName(), "MediaPlayerEditor.Tabs.Media"));
}

void FMediaPlaylistEditorToolkit::UnregisterTabSpawners(const TSharedRef<FTabManager>& InTabManager)
{
	FAssetEditorToolkit::UnregisterTabSpawners(InTabManager);

	InTabManager->UnregisterTabSpawner(MediaPlaylistEditorToolkit::MediaTabId);	
	InTabManager->UnregisterTabSpawner(MediaPlaylistEditorToolkit::DetailsTabId);
}

FText FMediaPlaylistEditorToolkit::GetBaseToolkitName() const
{
	return LOCTEXT("AppLabel", "Media Playlist Editor");
}

FName FMediaPlaylistEditorToolkit::GetToolkitFName() const
{
	return FName("MediaPlaylistEditor");
}

FString FMediaPlaylistEditorToolkit::GetWorldCentricTabPrefix() const
{
	return LOCTEXT("WorldCentricTabPrefix", "MediaPlaylist ").ToString();
}

void FMediaPlaylistEditorToolkit::AddReferencedObjects(FReferenceCollector& InCollector)
{
	InCollector.AddReferencedObject(MediaPlaylist);
}

TSharedRef<SDockTab> FMediaPlaylistEditorToolkit::HandleTabManagerSpawnTab(const FSpawnTabArgs& Args, FName TabIdentifier)
{
	TSharedPtr<SWidget> TabWidget = SNullWidget::NullWidget;

	if (TabIdentifier == MediaPlaylistEditorToolkit::DetailsTabId)
	{
		TabWidget = SNew(SMediaPlaylistEditorDetails, *MediaPlaylist, Style);
	}
	else if (TabIdentifier == MediaPlaylistEditorToolkit::MediaTabId)
	{
		TabWidget = SNew(SMediaPlaylistEditorMedia, *MediaPlaylist, Style);
	}

	return SNew(SDockTab)
		.TabRole(ETabRole::PanelTab)
		[
			TabWidget.ToSharedRef()
		];
}

#undef LOCTEXT_NAMESPACE
