// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Toolkits/MediaPlayerEditorToolkitBase.h"

class FSpawnTabArgs;
class SDockTab;
class UMediaPlaylist;

/**
 * Implements an Editor toolkit for media play lists.
 */
class FMediaPlaylistEditorToolkit
	: public FMediaPlayerEditorToolkitBase
{
public:
	/**
	 * Creates and initializes a new instance.
	 *
	 * @param InStyle The style set to use.
	 */
	FMediaPlaylistEditorToolkit(const TSharedRef<ISlateStyle>& InStyle);

	/**
	 * Initializes the editor tool kit.
	 *
	 * @param InMediaPlaylist The UMediaPlaylist asset to edit.
	 * @param InMode The mode to create the toolkit in.
	 * @param InToolkitHost The toolkit host.
	 */
	void Initialize(UMediaPlaylist* InMediaPlaylist, const EToolkitMode::Type InMode, const TSharedPtr<IToolkitHost>& InToolkitHost);

	//~ Begin FAssetEditorToolkit
	virtual void RegisterTabSpawners(const TSharedRef<FTabManager>& TabManager) override;
	virtual void UnregisterTabSpawners(const TSharedRef<FTabManager>& TabManager) override;
	//~ End FAssetEditorToolkit

	//~ Begin IToolkit
	virtual FText GetBaseToolkitName() const override;
	virtual FName GetToolkitFName() const override;
	virtual FString GetWorldCentricTabPrefix() const override;
	//~ End IToolkit

	//~ Begin FGCObject
	virtual void AddReferencedObjects(FReferenceCollector& InCollector) override;
	virtual FString GetReferencerName() const override
	{
		return TEXT("FMediaPlaylistEditorToolkit");
	}
	//~ End FGCObject

protected:
	//~ Begin FMediaPlayerToolkitBase
	virtual TSharedRef<FTabManager::FLayout> CreateLayout() override;
	//~ End FMediaPlayerToolkitBase

private:
	/** The media play list asset being edited. */
	TObjectPtr<UMediaPlaylist> MediaPlaylist;

	/** Callback for spawning tabs. */
	TSharedRef<SDockTab> HandleTabManagerSpawnTab(const FSpawnTabArgs& Args, FName TabIdentifier);
};
