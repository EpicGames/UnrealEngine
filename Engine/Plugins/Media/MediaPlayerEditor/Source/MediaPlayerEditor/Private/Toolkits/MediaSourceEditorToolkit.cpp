// Copyright Epic Games, Inc. All Rights Reserved.

#include "Toolkits/MediaSourceEditorToolkit.h"

#include "CommonRenderResources.h"
#include "Engine/TextureRenderTarget2D.h"
#include "Framework/Docking/TabManager.h"
#include "Framework/MultiBox/MultiBoxExtender.h"
#include "GlobalShader.h"
#include "IMediaModule.h"
#include "IMediaPlayerFactory.h"
#include "MediaPlayer.h"
#include "MediaSource.h"
#include "MediaTexture.h"
#include "Models/MediaPlayerEditorCommands.h"
#include "Modules/ModuleManager.h"
#include "PixelShaderUtils.h"
#include "PostProcess/DrawRectangle.h"
#include "RenderGraphBuilder.h"
#include "RenderGraphEvent.h"
#include "RenderGraphUtils.h"
#include "RHIResources.h"
#include "ScreenPass.h"
#include "Shader.h"
#include "ShaderParameterStruct.h"
#include "SlateOptMacros.h"
#include "TextureResource.h"
#include "Widgets/Docking/SDockTab.h"
#include "Widgets/SMediaImageTextureChannelToggle.h"
#include "Widgets/SMediaPlayerEditorDetails.h"
#include "Widgets/SMediaPlayerEditorMediaDetails.h"
#include "Widgets/SMediaPlayerEditorViewer.h"
#include "Widgets/SMediaSourceEditorDetails.h"
#include "UObject/UnrealType.h"
#include "UObject/UObjectGlobals.h"

#define LOCTEXT_NAMESPACE "FMediaSourceEditorToolkit"

namespace MediaSourceEditorToolkit
{
	static const FName AppIdentifier("MediaSourceEditorApp");
	static const FName DetailsTabId("Details");
	static const FName MediaDetailsTabId("MediaDetails");
	static const FName PlayerDetailsTabId("PlayerDetails");
	static const FName ViewerTabId("Viewer");
}

FMediaSourceEditorToolkit::FMediaSourceEditorToolkit(const TSharedRef<ISlateStyle>& InStyle)
	: FMediaPlayerEditorToolkitMediaPlayerBase(InStyle)
	, MediaSource(nullptr)
	, MediaTexture(nullptr)
{}

void FMediaSourceEditorToolkit::Initialize(UMediaSource* InMediaSource, const EToolkitMode::Type InMode, const TSharedPtr<IToolkitHost>& InToolkitHost)
{
	MediaSource = InMediaSource;

	if (MediaSource == nullptr)
	{
		return;
	}

	// support undo/redo
	MediaSource->SetFlags(RF_Transactional);

	MediaPlayer = NewObject<UMediaPlayer>(GetTransientPackage());
	MediaPlayer->SetLooping(true);
	MediaPlayer->PlayOnOpen = true;

	MediaTexture = NewObject<UMediaTexture>(GetTransientPackage(), NAME_None, RF_Transient | RF_Public);
	if (MediaTexture != nullptr)
	{
		MediaTexture->AutoClear = true;
		MediaTexture->SetMediaPlayer(MediaPlayer);
		MediaTexture->SetColorSpaceOverride(UE::Color::EColorSpace::sRGB);
		MediaTexture->UpdateResource();
	}

	FMediaPlayerEditorToolkitBase::Initialize(
		InMediaSource,
		MediaSourceEditorToolkit::AppIdentifier,
		InMode, 
		InToolkitHost
	);
}

TSharedRef<FTabManager::FLayout> FMediaSourceEditorToolkit::CreateLayout()
{
	return FTabManager::NewLayout("Standalone_MediaSourceEditor_v0.3")
		->AddArea
		(
			FTabManager::NewPrimaryArea()
				->SetOrientation(Orient_Horizontal)
				->Split
				(
					// viewer
					FTabManager::NewStack()
						->AddTab(MediaSourceEditorToolkit::ViewerTabId, ETabState::OpenedTab)
						->SetHideTabWell(true)
						->SetSizeCoefficient(0.6f)

				)
				->Split
				(
					FTabManager::NewSplitter()
						->SetOrientation(Orient_Vertical)
						->SetSizeCoefficient(0.4f)
						->Split
						(
							// Media details tab.
							FTabManager::NewStack()
							->AddTab(MediaSourceEditorToolkit::MediaDetailsTabId, ETabState::OpenedTab)
							->SetSizeCoefficient(0.2f)
						)
						->Split
						(
							// Details tab.
							FTabManager::NewStack()
								->AddTab(MediaSourceEditorToolkit::DetailsTabId, ETabState::OpenedTab)
								->AddTab(MediaSourceEditorToolkit::PlayerDetailsTabId, ETabState::OpenedTab)
								->SetForegroundTab(MediaSourceEditorToolkit::DetailsTabId)
								->SetSizeCoefficient(0.8f)
						)
				)
		);
}

void FMediaSourceEditorToolkit::RegisterTabSpawners(const TSharedRef<FTabManager>& InTabManager)
{
	WorkspaceMenuCategory = InTabManager->AddLocalWorkspaceMenuCategory(LOCTEXT("WorkspaceMenu_MediaSourceEditor", "Media Source Editor"));
	auto WorkspaceMenuCategoryRef = WorkspaceMenuCategory.ToSharedRef();

	FAssetEditorToolkit::RegisterTabSpawners(InTabManager);

	InTabManager->RegisterTabSpawner(MediaSourceEditorToolkit::DetailsTabId, FOnSpawnTab::CreateSP(this, &FMediaSourceEditorToolkit::HandleTabManagerSpawnTab, MediaSourceEditorToolkit::DetailsTabId))
		.SetDisplayName(LOCTEXT("DetailsTabName", "Details"))
		.SetGroup(WorkspaceMenuCategoryRef)
		.SetIcon(FSlateIcon(FAppStyle::GetAppStyleSetName(), "LevelEditor.Tabs.Details"));

	InTabManager->RegisterTabSpawner(MediaSourceEditorToolkit::MediaDetailsTabId, FOnSpawnTab::CreateSP(this, &FMediaSourceEditorToolkit::HandleTabManagerSpawnTab, MediaSourceEditorToolkit::MediaDetailsTabId))
		.SetDisplayName(LOCTEXT("MediaDetailsTabName", "Media Details"))
		.SetGroup(WorkspaceMenuCategoryRef)
		.SetIcon(FSlateIcon(Style->GetStyleSetName(), "MediaPlayerEditor.Tabs.Info"));

	InTabManager->RegisterTabSpawner(MediaSourceEditorToolkit::PlayerDetailsTabId, FOnSpawnTab::CreateSP(this, &FMediaSourceEditorToolkit::HandleTabManagerSpawnTab, MediaSourceEditorToolkit::PlayerDetailsTabId))
		.SetDisplayName(LOCTEXT("PlayerDetailsTabName", "Player Details"))
		.SetGroup(WorkspaceMenuCategoryRef)
		.SetIcon(FSlateIcon(FAppStyle::GetAppStyleSetName(), "LevelEditor.Tabs.Details"));

	InTabManager->RegisterTabSpawner(MediaSourceEditorToolkit::ViewerTabId, FOnSpawnTab::CreateSP(this, &FMediaSourceEditorToolkit::HandleTabManagerSpawnTab, MediaSourceEditorToolkit::ViewerTabId))
		.SetDisplayName(LOCTEXT("PlayerTabName", "Player"))
		.SetGroup(WorkspaceMenuCategoryRef)
		.SetIcon(FSlateIcon(Style->GetStyleSetName(), "MediaPlayerEditor.Tabs.Player"));
}

void FMediaSourceEditorToolkit::UnregisterTabSpawners(const TSharedRef<class FTabManager>& InTabManager)
{
	FAssetEditorToolkit::UnregisterTabSpawners(InTabManager);

	InTabManager->UnregisterTabSpawner(MediaSourceEditorToolkit::ViewerTabId);
	InTabManager->UnregisterTabSpawner(MediaSourceEditorToolkit::PlayerDetailsTabId);
	InTabManager->UnregisterTabSpawner(MediaSourceEditorToolkit::MediaDetailsTabId);
	InTabManager->UnregisterTabSpawner(MediaSourceEditorToolkit::DetailsTabId);
}

FText FMediaSourceEditorToolkit::GetBaseToolkitName() const
{
	return LOCTEXT("AppLabel", "Media Source Editor");
}

FName FMediaSourceEditorToolkit::GetToolkitFName() const
{
	return FName("MediaSourceEditor");
}

FString FMediaSourceEditorToolkit::GetWorldCentricTabPrefix() const
{
	return LOCTEXT("WorldCentricTabPrefix", "MediaSource ").ToString();
}

void FMediaSourceEditorToolkit::AddReferencedObjects(FReferenceCollector& InCollector)
{
	FMediaPlayerEditorToolkitMediaPlayerBase::AddReferencedObjects(InCollector);

	InCollector.AddReferencedObject(MediaSource);
	InCollector.AddReferencedObject(MediaTexture);
}

void FMediaSourceEditorToolkit::ValidateDesiredPlayer(UMediaPlayer* InMediaPlayer, UMediaSource* InMediaSource)
{
	// Validate desired player
	FName DesiredPlayerName = InMediaPlayer->GetDesiredPlayerName();
	if (DesiredPlayerName != NAME_None)
	{
		if (IMediaModule* MediaModule = FModuleManager::LoadModulePtr<IMediaModule>("Media"))
		{
			if (IMediaPlayerFactory* PlayerFactory = MediaModule->GetPlayerFactory(DesiredPlayerName))
			{
				// Check that the given factory can open the media source, if not, don't use it.
				if (!PlayerFactory->CanPlayUrl(InMediaSource->GetUrl(), InMediaSource))
				{
					InMediaPlayer->SetDesiredPlayerName(NAME_None);
				}
			}
		}
	}
}

void FMediaSourceEditorToolkit::BindCommands()
{
	FMediaPlayerEditorToolkitMediaPlayerBase::BindCommands();

	const FMediaPlayerEditorCommands& Commands = FMediaPlayerEditorCommands::Get();

	ToolkitCommands->MapAction(
		Commands.OpenMedia,
		FExecuteAction::CreateLambda([this]
			{
				FMediaPlayerOptions Options;
				Options.SetAllAsOptional();
				Options.InternalCustomOptions.Emplace(MediaPlayerOptionValues::Environment(), MediaPlayerOptionValues::Environment_Preview());
				Options.InternalCustomOptions.Emplace(MediaPlayerOptionValues::ParseTimecodeInfo(), FVariant());
				ValidateDesiredPlayer(MediaPlayer, MediaSource);
				MediaPlayer->OpenSourceWithOptions(MediaSource, Options);
			}),
		FCanExecuteAction::CreateLambda([this] { return true; })
	);

	ToolkitCommands->MapAction(
		Commands.GenerateThumbnail,
		FExecuteAction::CreateLambda([this] { GenerateThumbnail(); })
	);
}

BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION
void FMediaSourceEditorToolkit::ExtendToolBar()
{
	TSharedPtr<FExtender> ToolbarExtender = MakeShared<FExtender>();

	ToolbarExtender->AddToolBarExtension(
		"Asset",
		EExtensionHook::After,
		GetToolkitCommands(),
		FToolBarExtensionDelegate::CreateLambda(
			[this](FToolBarBuilder& ToolbarBuilder)
			{
				const FMediaPlayerEditorCommands& Commands = FMediaPlayerEditorCommands::Get();

				ToolbarBuilder.BeginSection("MediaControls");
				{
					ToolbarBuilder.AddToolBarButton(Commands.GenerateThumbnail);
				}
				ToolbarBuilder.EndSection();

				using namespace MediaPlayerEditor::MediaImage;

				ToolbarBuilder.BeginSection("TextureControls");
				{
					ToolbarBuilder.AddWidget(SNew(SMediaImageTextureChannelToggle, Viewer, ETextureChannelMask::Red, ToolkitCommands));
					ToolbarBuilder.AddWidget(SNew(SMediaImageTextureChannelToggle, Viewer, ETextureChannelMask::Green, ToolkitCommands));
					ToolbarBuilder.AddWidget(SNew(SMediaImageTextureChannelToggle, Viewer, ETextureChannelMask::Blue, ToolkitCommands));
					ToolbarBuilder.AddWidget(SNew(SMediaImageTextureChannelToggle, Viewer, ETextureChannelMask::Alpha, ToolkitCommands));
				}
				ToolbarBuilder.EndSection();
			}
		)
	);

	AddToolbarExtender(ToolbarExtender);
}
END_SLATE_FUNCTION_BUILD_OPTIMIZATION

void FMediaSourceEditorToolkit::GenerateThumbnail()
{
	// Create render target.
	UTextureRenderTarget2D* ThumbnailTexture =
		NewObject<UTextureRenderTarget2D>(GetTransientPackage());
	if (ThumbnailTexture != nullptr)
	{
		ThumbnailTexture->RenderTargetFormat = ETextureRenderTargetFormat::RTF_RGBA8;
		ThumbnailTexture->ClearColor = FLinearColor::Black;
		ThumbnailTexture->bAutoGenerateMips = false;
		ThumbnailTexture->InitAutoFormat(MediaTexture->GetWidth(), MediaTexture->GetHeight());
		ThumbnailTexture->UpdateResourceImmediate(true);

		// Enqueue render command to copy the media texture to the render target.
		ENQUEUE_RENDER_COMMAND(MediaSourceRenderThumbnail)(
			[this, ThumbnailTexture](FRHICommandListImmediate& RHICmdList)
		{
			FTextureResource* DestResource = ThumbnailTexture->GetResource();
			FTextureResource* SourceResource = MediaTexture->GetResource();
			FRHITexture* SourceTexture = SourceResource->GetTextureRHI();
			FRHITexture* DestTexture = DestResource->GetTextureRHI();

			FRDGBuilder GraphBuilder(RHICmdList);
			FRDGTextureRef RDGSourceTexture = RegisterExternalTexture(GraphBuilder, SourceTexture, TEXT("MediaSourceThumnbailSourceTexture"));
			FRDGTextureRef RDGDestTexture = RegisterExternalTexture(GraphBuilder, DestTexture, TEXT("MediaSourceThumnbailDestTexture"));

			FGlobalShaderMap* ShaderMap = GetGlobalShaderMap(GMaxRHIFeatureLevel);

			TShaderMapRef<FScreenPassVS> VertexShader(ShaderMap);
			TShaderMapRef<FCopyRectPS> PixelShader(ShaderMap);

			FCopyRectPS::FParameters* PixelShaderParameters = GraphBuilder.AllocParameters<FCopyRectPS::FParameters>();
			PixelShaderParameters->InputTexture = RDGSourceTexture;
			PixelShaderParameters->InputSampler = TStaticSamplerState<SF_Bilinear, AM_Clamp, AM_Clamp, AM_Clamp>::GetRHI();
			PixelShaderParameters->RenderTargets[0] = FRenderTargetBinding(RDGDestTexture, ERenderTargetLoadAction::ELoad);

			ClearUnusedGraphResources(PixelShader, PixelShaderParameters);

			FRHIBlendState* BlendState = FScreenPassPipelineState::FDefaultBlendState::GetRHI();
			FRHIDepthStencilState* DepthStencilState = FScreenPassPipelineState::FDefaultDepthStencilState::GetRHI();

			// Create the pipline state that will execute
			const FScreenPassPipelineState PipelineState(VertexShader, PixelShader, BlendState, DepthStencilState);
			GraphBuilder.AddPass(
				RDG_EVENT_NAME("MediaSourceThumbnailCopy"),
				PixelShaderParameters,
				ERDGPassFlags::Raster,
				[PipelineState, Extent = RDGSourceTexture->Desc.Extent, PixelShader, PixelShaderParameters](FRHICommandList& RHICmdList) {
				PipelineState.Validate();

				RHICmdList.SetViewport(0.0f, 0.0f, 0.0f, static_cast<float>(Extent.X), static_cast<float>(Extent.Y), 1.0f);
				SetScreenPassPipelineState(RHICmdList, PipelineState);
				SetShaderParameters(RHICmdList, PixelShader, PixelShader.GetPixelShader(), *PixelShaderParameters);

				FRHIBatchedShaderParameters& BatchedParameters = RHICmdList.GetScratchShaderParameters();
				UE::Renderer::PostProcess::SetDrawRectangleParameters(BatchedParameters, PipelineState.VertexShader.GetShader(),
					0.0f, 0.0f, static_cast<float>(Extent.X), static_cast<float>(Extent.Y),
					0.0f, 0.0f, static_cast<float>(Extent.X), static_cast<float>(Extent.Y),
					Extent,
					Extent);
				RHICmdList.SetBatchedShaderParameters(PipelineState.VertexShader.GetVertexShader(), BatchedParameters);
				FPixelShaderUtils::DrawFullscreenTriangle(RHICmdList);
			});

			GraphBuilder.Execute();

		});

		MediaSource->SetThumbnail(ThumbnailTexture);

		// Trigger a thumbnail render.
		FPropertyChangedEvent EmptyPropertyChangedEvent(nullptr);
		FCoreUObjectDelegates::OnObjectPropertyChanged.Broadcast(MediaSource,
			EmptyPropertyChangedEvent);
		MediaSource->MarkPackageDirty();
	}
}

TSharedRef<SDockTab> FMediaSourceEditorToolkit::HandleTabManagerSpawnTab(const FSpawnTabArgs& Args, FName TabIdentifier)
{
	TSharedPtr<SWidget> TabWidget = SNullWidget::NullWidget;

	if (TabIdentifier == MediaSourceEditorToolkit::DetailsTabId)
	{
		TabWidget = SNew(SMediaSourceEditorDetails, *MediaSource, Style);
	}
	else if (TabIdentifier == MediaSourceEditorToolkit::MediaDetailsTabId)
	{
		TabWidget = SNew(SMediaPlayerEditorMediaDetails, MediaPlayer, MediaTexture);
	}
	else if (TabIdentifier == MediaSourceEditorToolkit::PlayerDetailsTabId)
	{
		TabWidget = SNew(SMediaPlayerEditorDetails, *MediaPlayer, Style);
	}
	else if (TabIdentifier == MediaSourceEditorToolkit::ViewerTabId)
	{
		TabWidget = SAssignNew(Viewer, SMediaPlayerEditorViewer, *MediaPlayer, MediaTexture, Style, true)
			.bShowUrl(false)
			.Commands(ToolkitCommands);
	}

	return SNew(SDockTab)
		.TabRole(ETabRole::PanelTab)
		[
			TabWidget.ToSharedRef()
		];
}

#undef LOCTEXT_NAMESPACE
