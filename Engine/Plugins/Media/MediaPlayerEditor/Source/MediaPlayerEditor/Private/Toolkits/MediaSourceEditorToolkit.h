// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Toolkits/MediaPlayerEditorToolkitMediaPlayerBase.h"

class FSpawnTabArgs;
class SDockTab;
class UMediaPlayer;
class UMediaSource;
class UMediaTexture;

/**
 * Implements an Editor toolkit for media sources.
 */
class FMediaSourceEditorToolkit
	: public FMediaPlayerEditorToolkitMediaPlayerBase
{
public:
	/**
	 * Creates and initializes a new instance.
	 *
	 * @param InStyle The style set to use.
	 */
	FMediaSourceEditorToolkit(const TSharedRef<ISlateStyle>& InStyle);

	/**
	 * Initializes the editor tool kit.
	 *
	 * @param InMediaSource The UMediaSource asset to edit.
	 * @param InMode The mode to create the toolkit in.
	 * @param InToolkitHost The toolkit host.
	 */
	void Initialize(UMediaSource* InMediaSource, const EToolkitMode::Type InMode, const TSharedPtr<IToolkitHost>& InToolkitHost);

	//~ Begin FAssetEditorToolkit
	virtual void RegisterTabSpawners(const TSharedRef<FTabManager>& TabManager) override;
	virtual void UnregisterTabSpawners(const TSharedRef<FTabManager>& TabManager) override;
	//~ End FAssetEditorToolkit

	//~ Begin IToolkit
	virtual FText GetBaseToolkitName() const override;
	virtual FName GetToolkitFName() const override;
	virtual FString GetWorldCentricTabPrefix() const override;
	//~ End IToolkit

	//~ Begin FGCObject
	virtual void AddReferencedObjects(FReferenceCollector& InCollector) override;
	virtual FString GetReferencerName() const override
	{
		return TEXT("FMediaSourceEditorToolkit");
	}
	//~ End FGCObject

protected:
	//~ Begin FMediaPlayerToolkitBase
	virtual TSharedRef<FTabManager::FLayout> CreateLayout() override;
	virtual void BindCommands() override;
	virtual void ExtendToolBar() override;
	//~ End FMediaPlayerToolkitBase

private:
	/**
	 * Validates that the given player can open the media source with the selected desired player.
	 * If not, the desired player is reset to fallback to the media source selection.
	 */
	static void ValidateDesiredPlayer(UMediaPlayer* InMediaPlayer, UMediaSource* InMediaSource);

	/** The media source asset being edited. */
	TObjectPtr<UMediaSource> MediaSource;

	/** The media texture to output the media to. */
	TObjectPtr<UMediaTexture> MediaTexture;

	/** Callback for spawning tabs. */
	TSharedRef<SDockTab> HandleTabManagerSpawnTab(const FSpawnTabArgs& Args, FName TabIdentifier);

	/** Enqueues rendering commands to generate a thumbnail. */
	void GenerateThumbnail();
};
