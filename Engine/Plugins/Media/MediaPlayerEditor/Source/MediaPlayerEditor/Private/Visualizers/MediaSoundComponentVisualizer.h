// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AttenuatedComponentVisualizer.h"
#include "MediaSoundComponent.h"


/**
 * Visualizer for media sound components.
 */
class FMediaSoundComponentVisualizer
	: public TAttenuatedComponentVisualizer<UMediaSoundComponent>
{
private:

	virtual bool IsVisualizerEnabled(const FEngineShowFlags& ShowFlags) const override
	{
		return ShowFlags.AudioRadius;
	}
};
