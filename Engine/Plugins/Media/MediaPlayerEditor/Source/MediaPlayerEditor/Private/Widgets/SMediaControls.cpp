// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/SMediaControls.h"

#include "Framework/Commands/UICommandInfo.h"
#include "Framework/Commands/UICommandList.h"
#include "Internationalization/Text.h"
#include "MediaPlayer.h"
#include "MediaPlayerEditorModule.h"
#include "Models/MediaPlayerEditorCommands.h"
#include "Modules/ModuleManager.h"
#include "Shared/MediaPlayerEditorStyle.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/SBoxPanel.h"
#include "SMediaControls.h"

#define LOCTEXT_NAMESPACE "SMediaControls"

namespace MediaPlayerEditor
{
	namespace Private
	{
		FText GetKeybindText(const TSharedPtr<FUICommandInfo>& InCommand)
		{
			if (InCommand.IsValid())
			{
				const TSharedRef<const FInputChord>& Chord = InCommand->GetFirstValidChord();

				if (Chord->IsValidChord())
				{
					return Chord->GetInputText(/* Long display name */ false);
				}
			}

			return FText::GetEmpty();
		}

		FText GetToolTipWithKeybind(const TSharedRef<FUICommandInfo>& InCommand, TSharedPtr<FUICommandInfo> InBackupKeybindCommand = nullptr)
		{
			FText KeybindText = GetKeybindText(InCommand);

			if (KeybindText.IsEmpty())
			{
				KeybindText = GetKeybindText(InBackupKeybindCommand);
			}

			const FText& Description = InCommand->GetDescription();
			const FText& Text = Description.IsEmpty() ? InCommand->GetLabel() : Description;

			if (KeybindText.IsEmpty())
			{
				return Text;
			}

			return FText::Format(LOCTEXT("GetToolTipWithKeybind", "{0}\n\nKey: {1}"), Text, KeybindText);
		}
	}
}

void SMediaControls::Construct(const FArguments& InArgs, TSharedPtr<FUICommandList> InCommandList, UMediaPlayer* InMediaPlayer)
{
	CommandListWeak = InCommandList;
	PlayerWeak = InMediaPlayer;

	ChildSlot
	[
		CreateControls()
	];
}

TSharedRef<SWidget> SMediaControls::CreateControls()
{
	using namespace MediaPlayerEditor::Private;

	const FMediaPlayerEditorCommands& Commands = FMediaPlayerEditorCommands::Get();
	
	IMediaPlayerEditorModule& Module = FModuleManager::GetModuleChecked<IMediaPlayerEditorModule>("MediaPlayerEditor");	
	TSharedPtr<ISlateStyle> Style = Module.GetStyle();

	return SNew(SHorizontalBox)

		// Rewind button.
		+ SHorizontalBox::Slot()
		.VAlign(VAlign_Center)
		.AutoWidth()
		[
			SNew(SButton)
			.VAlign(VAlign_Center)
			.IsEnabled(this, &SMediaControls::Rewind_IsEnabled)
			.OnClicked(this, &SMediaControls::Rewind_OnClicked)
			.ButtonStyle(Style, "MediaPlayerEditor.MediaControlButton")
			.ContentPadding(2.f)
			[
				SNew(SImage)
				.ColorAndOpacity(FSlateColor::UseForeground())
				.Image(Style->GetBrush("MediaPlayerEditor.RewindMedia"))
				.ToolTipText(GetToolTipWithKeybind(Commands.RewindMedia.ToSharedRef()))
				.DesiredSizeOverride(FVector2D(20.f))
			]
		]
		
		// Reverse button.
		/*
		+ SHorizontalBox::Slot()
		.VAlign(VAlign_Center)
		.AutoWidth()
		[
			SNew(SButton)
			.VAlign(VAlign_Center)
			.IsEnabled(this, &SMediaControls::Reverse_IsEnabled)
			.OnClicked(this, &SMediaControls::Reverse_OnClicked)
			.ButtonStyle(Style, "MediaPlayerEditor.MediaControlButton")
			.ContentPadding(2.f)
			[
				SNew(SImage)
				.ColorAndOpacity(FSlateColor::UseForeground())
				.Image(Style->GetBrush("MediaPlayerEditor.ReverseMedia"))
				.ToolTipText(GetToolTipWithKeybind(Commands.ReverseMedia.ToSharedRef()))
				.DesiredSizeOverride(FVector2D(20.f))
			]
		]
		*/
		
		// Step back button.
		+ SHorizontalBox::Slot()
		.VAlign(VAlign_Center)
		.AutoWidth()
		[
			SNew(SButton)
			.VAlign(VAlign_Center)
			.IsEnabled(this, &SMediaControls::StepBack_IsEnabled)
			.OnClicked(this, &SMediaControls::StepBack_OnClicked)
			.ButtonStyle(Style, "MediaPlayerEditor.MediaControlButton")
			.ContentPadding(2.f)
			[
				SNew(SImage)
				.ColorAndOpacity(FSlateColor::UseForeground())
				.Image(Style->GetBrush("MediaPlayerEditor.StepBackwardMedia"))
				.ToolTipText(GetToolTipWithKeybind(Commands.StepBackwardMedia.ToSharedRef()))
				.DesiredSizeOverride(FVector2D(20.f))
			]
		]

		// Reverse button.
		+ SHorizontalBox::Slot()
		.VAlign(VAlign_Center)
		.AutoWidth()
		[
			SNew(SButton)
			.VAlign(VAlign_Center)
			.IsEnabled(this, &SMediaControls::PlayReverse_IsEnabled)
			.OnClicked(this, &SMediaControls::PlayReverse_OnClicked)
			.ButtonStyle(Style, "MediaPlayerEditor.MediaControlButton")
			.ContentPadding(2.f)
			[
				SNew(SImage)
				.ColorAndOpacity(FSlateColor::UseForeground())
				.Image(this, &SMediaControls::PlayReverse_GetBrush)
				.ToolTipText(this, &SMediaControls::PlayReverse_GetToolTip)
				.DesiredSizeOverride(FVector2D(20.f))
			]
		]

		// Play button.
		+ SHorizontalBox::Slot()
		.VAlign(VAlign_Center)
		.AutoWidth()
		[
			SNew(SButton)
			.VAlign(VAlign_Center)
			.IsEnabled(this, &SMediaControls::Play_IsEnabled)
			.OnClicked(this, &SMediaControls::Play_OnClicked)
			.ButtonStyle(Style, "MediaPlayerEditor.MediaControlButton")
			.ContentPadding(2.f)
			[
				SNew(SImage)
				.ColorAndOpacity(FSlateColor::UseForeground())
				.Image(this, &SMediaControls::Play_GetBrush)
				.ToolTipText(this, &SMediaControls::Play_GetToolTip)
				.DesiredSizeOverride(FVector2D(20.f))
			]
		]

		// Step forward button.
		+ SHorizontalBox::Slot()
		.VAlign(VAlign_Center)
		.AutoWidth()
		[
			SNew(SButton)
			.VAlign(VAlign_Center)
			.IsEnabled(this, &SMediaControls::StepForwardMedia_IsEnabled)
			.OnClicked(this, &SMediaControls::StepForwardMedia_OnClicked)
			.ButtonStyle(Style, "MediaPlayerEditor.MediaControlButton")
			.ContentPadding(2.f)
			[
				SNew(SImage)
				.ColorAndOpacity(FSlateColor::UseForeground())
				.Image(Style->GetBrush("MediaPlayerEditor.StepForwardMedia"))
				.ToolTipText(GetToolTipWithKeybind(Commands.StepForwardMedia.ToSharedRef()))
				.DesiredSizeOverride(FVector2D(20.f))
			]
		]
		
		// Fast forward button.
		/*
		+ SHorizontalBox::Slot()
		.VAlign(VAlign_Center)
		.AutoWidth()
		[
			SNew(SButton)
			.VAlign(VAlign_Center)
			.IsEnabled(this, &SMediaControls::Forward_IsEnabled)
			.OnClicked(this, &SMediaControls::Forward_OnClicked)
			.ButtonStyle(Style, "MediaPlayerEditor.MediaControlButton")
			.ContentPadding(2.f)
			[
				SNew(SImage)
				.ColorAndOpacity(FSlateColor::UseForeground())
				.Image(Style->GetBrush("MediaPlayerEditor.ForwardMedia"))
				.ToolTipText(GetToolTipWithKeybind(Commands.ForwardMedia.ToSharedRef()))
				.DesiredSizeOverride(FVector2D(20.f))
			]
		]
		*/

		// Jump to end button.
		+ SHorizontalBox::Slot()
		.VAlign(VAlign_Center)
		.AutoWidth()
		[
			SNew(SButton)
			.VAlign(VAlign_Center)
			.IsEnabled(this, &SMediaControls::JumpToEndMedia_IsEnabled)
			.OnClicked(this, &SMediaControls::JumpToEndMedia_OnClicked)
			.ButtonStyle(Style, "MediaPlayerEditor.MediaControlButton")
			.ContentPadding(2.f)
			[
				SNew(SImage)
				.ColorAndOpacity(FSlateColor::UseForeground())
				.Image(Style->GetBrush("MediaPlayerEditor.JumpToEndMedia"))
				.ToolTipText(GetToolTipWithKeybind(Commands.JumpToEndMedia.ToSharedRef()))
				.DesiredSizeOverride(FVector2D(20.f))
			]
		]

		// Previous button
		+ SHorizontalBox::Slot()
		.VAlign(VAlign_Center)
		.AutoWidth()
		.Padding(20.f, 0.f, 0.f, 0.f)
		[
			SNew(SButton)
			.VAlign(VAlign_Center)
			.IsEnabled(this, &SMediaControls::PreviousMedia_IsEnabled)
			.OnClicked(this, &SMediaControls::PreviousMedia_OnClicked)
			.ButtonStyle(Style, "MediaPlayerEditor.MediaControlButton")
			.ContentPadding(2.f)
			[
				SNew(SImage)
				.ColorAndOpacity(FSlateColor::UseForeground())
				.Image(Style->GetBrush("MediaPlayerEditor.PreviousMedia"))
				.ToolTipText(GetToolTipWithKeybind(Commands.PreviousMedia.ToSharedRef()))
				.DesiredSizeOverride(FVector2D(20.f))
			]
		]

		// Next button
		+ SHorizontalBox::Slot()
		.VAlign(VAlign_Center)
		.AutoWidth()
		[
			SNew(SButton)
			.VAlign(VAlign_Center)
			.IsEnabled(this, &SMediaControls::NextMedia_IsEnabled)
			.OnClicked(this, &SMediaControls::NextMedia_OnClicked)
			.ButtonStyle(Style, "MediaPlayerEditor.MediaControlButton")
			.ContentPadding(2.f)
			[
				SNew(SImage)
				.ColorAndOpacity(FSlateColor::UseForeground())
				.Image(Style->GetBrush("MediaPlayerEditor.NextMedia"))
				.ToolTipText(GetToolTipWithKeybind(Commands.NextMedia.ToSharedRef()))
				.DesiredSizeOverride(FVector2D(20.f))
			]
		]

		// Open/Close Media
		+ SHorizontalBox::Slot()
		.VAlign(VAlign_Center)
		.AutoWidth()
		[
			SNew(SButton)
			.VAlign(VAlign_Center)
			.IsEnabled(this, &SMediaControls::OpenCloseMedia_IsEnabled)
			.OnClicked(this, &SMediaControls::OpenCloseMedia_OnClicked)
			.ButtonStyle(Style, "MediaPlayerEditor.MediaControlButton")
			.ContentPadding(2.f)
			[
				SNew(SImage)
				.ColorAndOpacity(FSlateColor::UseForeground())
				.Image(this, &SMediaControls::OpenCloseMedia_GetBrush)
				.ToolTipText(this, &SMediaControls::OpenCloseMedia_GetToolTip)
				.DesiredSizeOverride(FVector2D(20.f))
			]
		];
}

bool SMediaControls::PreviousMedia_IsEnabled() const
{
	if (TSharedPtr<FUICommandList> CommandList = CommandListWeak.Pin())
	{
		return CommandList->CanExecuteAction(FMediaPlayerEditorCommands::Get().PreviousMedia.ToSharedRef());
	}

	return false;
}

FReply SMediaControls::PreviousMedia_OnClicked()
{
	if (TSharedPtr<FUICommandList> CommandList = CommandListWeak.Pin())
	{
		CommandList->TryExecuteAction(FMediaPlayerEditorCommands::Get().PreviousMedia.ToSharedRef());
	}

	return FReply::Handled();
}

bool SMediaControls::Rewind_IsEnabled() const
{
	if (TSharedPtr<FUICommandList> CommandList = CommandListWeak.Pin())
	{
		return CommandList->CanExecuteAction(FMediaPlayerEditorCommands::Get().RewindMedia.ToSharedRef());
	}

	return false;
}

FReply SMediaControls::Rewind_OnClicked()
{
	if (TSharedPtr<FUICommandList> CommandList = CommandListWeak.Pin())
	{
		CommandList->TryExecuteAction(FMediaPlayerEditorCommands::Get().RewindMedia.ToSharedRef());
	}

	return FReply::Handled();
}

bool SMediaControls::PlayReverse_IsEnabled() const
{
	if (TSharedPtr<FUICommandList> CommandList = CommandListWeak.Pin())
	{
		return CommandList->CanExecuteAction(FMediaPlayerEditorCommands::Get().TogglePlayReversePauseMedia.ToSharedRef());
	}

	return false;
}

const FSlateBrush* SMediaControls::PlayReverse_GetBrush() const
{
	IMediaPlayerEditorModule& Module = FModuleManager::GetModuleChecked<IMediaPlayerEditorModule>("MediaPlayerEditor");
	TSharedPtr<ISlateStyle> Style = Module.GetStyle();

	if (UMediaPlayer* MediaPlayer = PlayerWeak.Get())
	{
		const float Rate = MediaPlayer->GetRate();

		if (Rate < 0 && !FMath::IsNearlyZero(Rate))
		{
			return Style->GetBrush("MediaPlayerEditor.PauseMedia");
		}
	}

	return Style->GetBrush("MediaPlayerEditor.PlayReverseMedia");
}

FText SMediaControls::PlayReverse_GetToolTip() const
{
	using namespace MediaPlayerEditor::Private;

	const FMediaPlayerEditorCommands& Commands = FMediaPlayerEditorCommands::Get();

	if (UMediaPlayer* MediaPlayer = PlayerWeak.Get())
	{
		const float Rate = MediaPlayer->GetRate();

		if (Rate < 0 && !FMath::IsNearlyZero(Rate))
		{
			return GetToolTipWithKeybind(Commands.PauseMedia.ToSharedRef(), Commands.TogglePlayReversePauseMedia);
		}
	}

	return GetToolTipWithKeybind(Commands.PlayReverseMedia.ToSharedRef(), Commands.TogglePlayReversePauseMedia);
}

FReply SMediaControls::PlayReverse_OnClicked()
{
	if (TSharedPtr<FUICommandList> CommandList = CommandListWeak.Pin())
	{
		CommandList->TryExecuteAction(FMediaPlayerEditorCommands::Get().TogglePlayReversePauseMedia.ToSharedRef());
	}

	return FReply::Handled();
}

bool SMediaControls::Reverse_IsEnabled() const
{
	if (TSharedPtr<FUICommandList> CommandList = CommandListWeak.Pin())
	{
		return CommandList->CanExecuteAction(FMediaPlayerEditorCommands::Get().ReverseMedia.ToSharedRef());
	}

	return false;
}

FReply SMediaControls::Reverse_OnClicked()
{
	if (TSharedPtr<FUICommandList> CommandList = CommandListWeak.Pin())
	{
		CommandList->TryExecuteAction(FMediaPlayerEditorCommands::Get().ReverseMedia.ToSharedRef());
	}

	return FReply::Handled();
}

bool SMediaControls::StepBack_IsEnabled() const
{
	if (TSharedPtr<FUICommandList> CommandList = CommandListWeak.Pin())
	{
		return CommandList->CanExecuteAction(FMediaPlayerEditorCommands::Get().StepBackwardMedia.ToSharedRef());
	}

	return false;
}

FReply SMediaControls::StepBack_OnClicked()
{
	if (TSharedPtr<FUICommandList> CommandList = CommandListWeak.Pin())
	{
		CommandList->TryExecuteAction(FMediaPlayerEditorCommands::Get().StepBackwardMedia.ToSharedRef());
	}

	return FReply::Handled();
}

bool SMediaControls::Play_IsEnabled() const
{
	if (TSharedPtr<FUICommandList> CommandList = CommandListWeak.Pin())
	{
		return CommandList->CanExecuteAction(FMediaPlayerEditorCommands::Get().TogglePlayPauseMedia.ToSharedRef());
	}

	return false;
}

const FSlateBrush* SMediaControls::Play_GetBrush() const
{
	IMediaPlayerEditorModule& Module = FModuleManager::GetModuleChecked<IMediaPlayerEditorModule>("MediaPlayerEditor");
	TSharedPtr<ISlateStyle> Style = Module.GetStyle();

	if (UMediaPlayer* MediaPlayer = PlayerWeak.Get())
	{
		const float Rate = MediaPlayer->GetRate();

		if (Rate > 0 && !FMath::IsNearlyZero(Rate))
		{
			return Style->GetBrush("MediaPlayerEditor.PauseMedia");
		}
	}

	return Style->GetBrush("MediaPlayerEditor.PlayMedia");
}

FText SMediaControls::Play_GetToolTip() const
{
	using namespace MediaPlayerEditor::Private;

	const FMediaPlayerEditorCommands& Commands = FMediaPlayerEditorCommands::Get();

	if (UMediaPlayer* MediaPlayer = PlayerWeak.Get())
	{
		const float Rate = MediaPlayer->GetRate();

		if (Rate > 0 && !FMath::IsNearlyZero(Rate))
		{
			return GetToolTipWithKeybind(Commands.PauseMedia.ToSharedRef(), Commands.TogglePlayPauseMedia);
		}
	}

	return GetToolTipWithKeybind(Commands.PlayMedia.ToSharedRef(), Commands.TogglePlayPauseMedia);
}

FReply SMediaControls::Play_OnClicked()
{
	if (TSharedPtr<FUICommandList> CommandList = CommandListWeak.Pin())
	{
		CommandList->TryExecuteAction(FMediaPlayerEditorCommands::Get().TogglePlayPauseMedia.ToSharedRef());
	}

	return FReply::Handled();
}

bool SMediaControls::StepForwardMedia_IsEnabled() const
{
	if (TSharedPtr<FUICommandList> CommandList = CommandListWeak.Pin())
	{
		return CommandList->CanExecuteAction(FMediaPlayerEditorCommands::Get().StepForwardMedia.ToSharedRef());
	}

	return false;
}

FReply SMediaControls::StepForwardMedia_OnClicked()
{
	if (TSharedPtr<FUICommandList> CommandList = CommandListWeak.Pin())
	{
		CommandList->TryExecuteAction(FMediaPlayerEditorCommands::Get().StepForwardMedia.ToSharedRef());
	}

	return FReply::Handled();
}

bool SMediaControls::Forward_IsEnabled() const
{
	if (TSharedPtr<FUICommandList> CommandList = CommandListWeak.Pin())
	{
		return CommandList->CanExecuteAction(FMediaPlayerEditorCommands::Get().ForwardMedia.ToSharedRef());
	}

	return false;
}

FReply SMediaControls::Forward_OnClicked()
{
	if (TSharedPtr<FUICommandList> CommandList = CommandListWeak.Pin())
	{
		CommandList->TryExecuteAction(FMediaPlayerEditorCommands::Get().ForwardMedia.ToSharedRef());
	}

	return FReply::Handled();
}

bool SMediaControls::JumpToEndMedia_IsEnabled() const
{
	if (TSharedPtr<FUICommandList> CommandList = CommandListWeak.Pin())
	{
		return CommandList->CanExecuteAction(FMediaPlayerEditorCommands::Get().JumpToEndMedia.ToSharedRef());
	}

	return false;
}

FReply SMediaControls::JumpToEndMedia_OnClicked()
{
	if (TSharedPtr<FUICommandList> CommandList = CommandListWeak.Pin())
	{
		CommandList->TryExecuteAction(FMediaPlayerEditorCommands::Get().JumpToEndMedia.ToSharedRef());
	}

	return FReply::Handled();
}

bool SMediaControls::NextMedia_IsEnabled() const
{
	if (TSharedPtr<FUICommandList> CommandList = CommandListWeak.Pin())
	{
		return CommandList->CanExecuteAction(FMediaPlayerEditorCommands::Get().NextMedia.ToSharedRef());
	}

	return false;
}

FReply SMediaControls::NextMedia_OnClicked()
{
	if (TSharedPtr<FUICommandList> CommandList = CommandListWeak.Pin())
	{
		CommandList->TryExecuteAction(FMediaPlayerEditorCommands::Get().NextMedia.ToSharedRef());
	}

	return FReply::Handled();
}

bool SMediaControls::OpenCloseMedia_IsEnabled() const
{
	if (TSharedPtr<FUICommandList> CommandList = CommandListWeak.Pin())
	{
		const FMediaPlayerEditorCommands& Commands = FMediaPlayerEditorCommands::Get();

		if (UMediaPlayer* MediaPlayer = PlayerWeak.Get())
		{
			if (MediaPlayer->IsReady())
			{
				return CommandList->CanExecuteAction(Commands.CloseMedia.ToSharedRef());
			}
		}

		return CommandList->CanExecuteAction(Commands.OpenMedia.ToSharedRef());
	}

	return false;
}

const FSlateBrush* SMediaControls::OpenCloseMedia_GetBrush() const
{
	IMediaPlayerEditorModule& Module = FModuleManager::GetModuleChecked<IMediaPlayerEditorModule>("MediaPlayerEditor");
	TSharedPtr<ISlateStyle> Style = Module.GetStyle();

	if (UMediaPlayer* MediaPlayer = PlayerWeak.Get())
	{
		if (MediaPlayer->IsReady())
		{
			return Style->GetBrush("MediaPlayerEditor.CloseMedia");
		}
	}

	return Style->GetBrush("MediaPlayerEditor.OpenMedia");
}

FText SMediaControls::OpenCloseMedia_GetToolTip() const
{
	using namespace MediaPlayerEditor::Private;

	if (UMediaPlayer* MediaPlayer = PlayerWeak.Get())
	{
		if (MediaPlayer->IsReady())
		{
			return GetToolTipWithKeybind(FMediaPlayerEditorCommands::Get().CloseMedia.ToSharedRef());
		}
	}

	return GetToolTipWithKeybind(FMediaPlayerEditorCommands::Get().OpenMedia.ToSharedRef());
}

FReply SMediaControls::OpenCloseMedia_OnClicked()
{
	if (TSharedPtr<FUICommandList> CommandList = CommandListWeak.Pin())
	{
		if (UMediaPlayer* MediaPlayer = PlayerWeak.Get())
		{
			if (MediaPlayer->IsReady())
			{
				CommandList->TryExecuteAction(FMediaPlayerEditorCommands::Get().CloseMedia.ToSharedRef());
				return FReply::Handled();
			}
		}

		CommandList->TryExecuteAction(FMediaPlayerEditorCommands::Get().OpenMedia.ToSharedRef());
	}

	return FReply::Handled();
}

#undef LOCTEXT_NAMESPACE
