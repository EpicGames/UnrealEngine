// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/SCompoundWidget.h"

#include "UObject/WeakObjectPtr.h"

class FText;
class FUICommandList;
class UMediaPlayer;

class SMediaControls : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SMediaControls) 
		{}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, TSharedPtr<FUICommandList> InCommandList, UMediaPlayer* InMediaPlayer);

protected:
	TWeakPtr<FUICommandList> CommandListWeak;
	TWeakObjectPtr<UMediaPlayer> PlayerWeak;

	TSharedRef<SWidget> CreateControls();

	bool PreviousMedia_IsEnabled() const;
	FReply PreviousMedia_OnClicked();

	bool Rewind_IsEnabled() const;
	FReply Rewind_OnClicked();

	bool PlayReverse_IsEnabled() const;
	const FSlateBrush* PlayReverse_GetBrush() const;
	FText PlayReverse_GetToolTip() const;
	FReply PlayReverse_OnClicked();

	bool Reverse_IsEnabled() const;
	FReply Reverse_OnClicked();

	bool StepBack_IsEnabled() const;
	FReply StepBack_OnClicked();

	bool Play_IsEnabled() const;
	const FSlateBrush* Play_GetBrush() const;
	FText Play_GetToolTip() const;
	FReply Play_OnClicked();

	bool StepForwardMedia_IsEnabled() const;
	FReply StepForwardMedia_OnClicked();

	bool Forward_IsEnabled() const;
	FReply Forward_OnClicked();

	bool JumpToEndMedia_IsEnabled() const;
	FReply JumpToEndMedia_OnClicked();

	bool NextMedia_IsEnabled() const;
	FReply NextMedia_OnClicked();

	bool OpenCloseMedia_IsEnabled() const;
	const FSlateBrush* OpenCloseMedia_GetBrush() const;
	FText OpenCloseMedia_GetToolTip() const;
	FReply OpenCloseMedia_OnClicked();
};