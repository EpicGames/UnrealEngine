// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/SMediaImage.h"

#include "Engine/Texture.h"
#include "MaterialDomain.h"
#include "Materials/Material.h"
#include "Materials/MaterialExpressionConvert.h"
#include "Materials/MaterialExpressionSubstrate.h"
#include "Materials/MaterialExpressionTextureObjectParameter.h"
#include "Materials/MaterialExpressionTextureSample.h"
#include "Materials/MaterialInstanceDynamic.h"
#include "RenderUtils.h"
#include "Styling/AppStyle.h"
#include "Styling/SlateBrush.h"
#include "UObject/Package.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Layout/SScaleBox.h"

namespace MediaPlayerEditor
{
	namespace MediaImage::Private
	{
		static const FLazyName TextureParameterName = "Texture";

		UMaterial* CreateChannelMaskMaterial(UTexture* InTexture, MediaPlayerEditor::MediaImage::ETextureChannelMask InMask)
		{
			UMaterial* Material = NewObject<UMaterial>(GetTransientPackage(), NAME_None, RF_Transient);

			if (!Material)
			{
				return {};
			}

			UMaterialEditorOnlyData* MaterialEditorOnly = Material->GetEditorOnlyData();

			UMaterialExpressionTextureObjectParameter* TextureParameter = NewObject<UMaterialExpressionTextureObjectParameter>(Material);
			{
				TextureParameter->SetParameterName(TextureParameterName);
				TextureParameter->SamplerSource = ESamplerSourceMode::SSM_Clamp_WorldGroupSettings;
			}

			UMaterialExpressionTextureSample* TextureSampler = NewObject<UMaterialExpressionTextureSample>(Material);
			{
				TextureSampler->Texture = InTexture;
				TextureSampler->AutoSetSampleType();
				TextureSampler->Texture = nullptr;

				TextureSampler->TextureObject.Connect(0, TextureParameter);
			}

			using namespace MediaPlayerEditor::MediaImage;

			UMaterialExpressionConvert* ConvertNode = NewObject<UMaterialExpressionConvert>(Material);
			{
				FMaterialExpressionConvertInput& RedInput = ConvertNode->ConvertInputs.AddDefaulted_GetRef();
				RedInput.Type = EMaterialExpressionConvertType::Scalar;
				
				if (EnumHasAnyFlags(InMask, ETextureChannelMask::Red))
				{
					RedInput.ExpressionInput.Connect(1, TextureSampler);
				}

				FMaterialExpressionConvertInput& GreenInput = ConvertNode->ConvertInputs.AddDefaulted_GetRef();
				GreenInput.Type = EMaterialExpressionConvertType::Scalar;

				if (EnumHasAnyFlags(InMask, ETextureChannelMask::Green))
				{
					GreenInput.ExpressionInput.Connect(2, TextureSampler);
				}

				FMaterialExpressionConvertInput& BlueInput = ConvertNode->ConvertInputs.AddDefaulted_GetRef();
				BlueInput.Type = EMaterialExpressionConvertType::Scalar;

				if (EnumHasAnyFlags(InMask, ETextureChannelMask::Blue))
				{
					BlueInput.ExpressionInput.Connect(3, TextureSampler);
				}

				FMaterialExpressionConvertOutput& ConvertOutput = ConvertNode->ConvertOutputs.AddDefaulted_GetRef();
				ConvertOutput.Type = EMaterialExpressionConvertType::Vector3;

				ConvertNode->ConvertMappings.Add({ 0, 0, 0, 0 });
				ConvertNode->ConvertMappings.Add({ 1, 0, 0, 1 });
				ConvertNode->ConvertMappings.Add({ 2, 0, 0, 2 });
			}

			FExpressionOutput& Output = TextureSampler->GetOutputs()[0];
			if (Substrate::IsSubstrateEnabled())
			{
				UMaterialExpressionSubstrateUI* UINode = NewObject<UMaterialExpressionSubstrateUI>(Material);
				UINode->Material = Material;

				UINode->Color.Expression = ConvertNode;
				UINode->Color.Mask = 1;
				UINode->Color.MaskR = 1;
				UINode->Color.MaskG = 1;
				UINode->Color.MaskB = 1;
				UINode->Color.MaskA = 0;

				if (EnumHasAnyFlags(InMask, ETextureChannelMask::Alpha))
				{
					UINode->Opacity.Expression = TextureSampler;
					UINode->Opacity.Mask = 1;
					UINode->Opacity.MaskR = 0;
					UINode->Opacity.MaskG = 0;
					UINode->Opacity.MaskB = 0;
					UINode->Opacity.MaskA = 1;
				}

				MaterialEditorOnly->FrontMaterial.Connect(0, UINode);
			}
			else
			{
				FExpressionInput& Input = MaterialEditorOnly->EmissiveColor;
				{
					Input.Expression = ConvertNode;
					Input.Mask = 1;
					Input.MaskR = 1;
					Input.MaskG = 1;
					Input.MaskB = 1;
					Input.MaskA = 0;
				}

				if (EnumHasAnyFlags(InMask, ETextureChannelMask::Alpha))
				{
					FExpressionInput& Opacity = MaterialEditorOnly->Opacity;
					Opacity.Expression = TextureSampler;
					Opacity.Mask = 1;
					Opacity.MaskR = 0;
					Opacity.MaskG = 0;
					Opacity.MaskB = 0;
					Opacity.MaskA = 1;
				}
			}

			Material->BlendMode = BLEND_AlphaComposite;

			Material->GetExpressionCollection().AddExpression(TextureParameter);
			Material->GetExpressionCollection().AddExpression(TextureSampler);
			Material->GetExpressionCollection().AddExpression(ConvertNode);
			Material->MaterialDomain = EMaterialDomain::MD_UI;
			Material->PostEditChange();

			return Material;
		}

		UMaterial* CreateGreyscaleMaterial(UTexture* InTexture, MediaPlayerEditor::MediaImage::ETextureChannelMask InChannel)
		{
			UMaterial* Material = NewObject<UMaterial>(GetTransientPackage(), NAME_None, RF_Transient);

			if (!Material)
			{
				return {};
			}

			UMaterialEditorOnlyData* MaterialEditorOnly = Material->GetEditorOnlyData();

			UMaterialExpressionTextureObjectParameter* TextureParameter = NewObject<UMaterialExpressionTextureObjectParameter>(Material);
			{
				TextureParameter->SetParameterName(TextureParameterName);
				TextureParameter->SamplerSource = ESamplerSourceMode::SSM_Clamp_WorldGroupSettings;
			}

			UMaterialExpressionTextureSample* TextureSampler = NewObject<UMaterialExpressionTextureSample>(Material);
			{
				TextureSampler->Texture = InTexture;
				TextureSampler->AutoSetSampleType();
				TextureSampler->Texture = nullptr;

				TextureSampler->TextureObject.Connect(0, TextureParameter);
			}

			using namespace MediaPlayerEditor::MediaImage;

			UMaterialExpressionConvert* ConvertNode = NewObject<UMaterialExpressionConvert>(Material);
			{
				FMaterialExpressionConvertInput& ConvertInput = ConvertNode->ConvertInputs.AddDefaulted_GetRef();
				ConvertInput.Type = EMaterialExpressionConvertType::Vector3;

				switch (InChannel)
				{
					case ETextureChannelMask::Red:
						ConvertInput.ExpressionInput.Connect(1, TextureSampler);
						break;

					case ETextureChannelMask::Green:
						ConvertInput.ExpressionInput.Connect(2, TextureSampler);
						break;

					case ETextureChannelMask::Blue:
						ConvertInput.ExpressionInput.Connect(3, TextureSampler);
						break;

					case ETextureChannelMask::Alpha:
						ConvertInput.ExpressionInput.Connect(4, TextureSampler);
						break;

					default:
						return {};
				}

				FMaterialExpressionConvertOutput& ConvertOutput = ConvertNode->ConvertOutputs.AddDefaulted_GetRef();
				ConvertOutput.Type = EMaterialExpressionConvertType::Vector3;

				ConvertNode->ConvertMappings.Add({ 0, 0, 0, 0 }); // Input -> Red
				ConvertNode->ConvertMappings.Add({ 0, 0, 0, 1 }); // Input -> Green
				ConvertNode->ConvertMappings.Add({ 0, 0, 0, 2 }); // Input -> Blue
			}		

			if (Substrate::IsSubstrateEnabled())
			{
				UMaterialExpressionSubstrateUI* UINode = NewObject<UMaterialExpressionSubstrateUI>(Material);
				UINode->Material = Material;

				UINode->Color.Expression = ConvertNode;
				UINode->Color.Mask = 1;
				UINode->Color.MaskR = 1;
				UINode->Color.MaskG = 1;
				UINode->Color.MaskB = 1;
				UINode->Color.MaskA = 0;

				MaterialEditorOnly->FrontMaterial.Connect(0, UINode);
			}
			else
			{
				FExpressionInput& Input = MaterialEditorOnly->EmissiveColor;
				{
					Input.Expression = ConvertNode;
					Input.Mask = 1;
					Input.MaskR = 1;
					Input.MaskG = 1;
					Input.MaskB = 1;
					Input.MaskA = 0;
				}
			}

			Material->BlendMode = BLEND_Opaque;

			Material->GetExpressionCollection().AddExpression(TextureParameter);
			Material->GetExpressionCollection().AddExpression(TextureSampler);
			Material->GetExpressionCollection().AddExpression(ConvertNode);
			Material->MaterialDomain = EMaterialDomain::MD_UI;
			Material->PostEditChange();

			return Material;
		}

		UMaterialInstanceDynamic* CreateInstance(UMaterial* InBaseMaterial, UTexture* InTexture)
		{
			if (!InBaseMaterial || !InTexture)
			{
				return nullptr;
			}

			UMaterialInstanceDynamic* Instance = UMaterialInstanceDynamic::Create(InBaseMaterial, GetTransientPackage());
			Instance->SetTextureParameterValue(TextureParameterName, InTexture);

			return Instance;
		}
	}
}

TMap<MediaPlayerEditor::MediaImage::ETextureChannelMask, TStrongObjectPtr<UMaterial>> SMediaImage::CachedMaterials;

SMediaImage::SMediaImage()
	: Mask(MediaPlayerEditor::MediaImage::ETextureChannelMask::RGBA)
{}

void SMediaImage::Construct(const FArguments& InArgs, UTexture* InTexture)
{
	TextureWeak = InTexture;
	
	// The Slate brush that renders the material.
	BrushImageSize = InArgs._BrushImageSize;

	if (InTexture != nullptr)
	{
		// create Slate brush
		MaterialBrush = MakeShareable(new FSlateBrush());

		// Create material and assign to brush.
		SetChannelMask(Mask);
	}

	ChildSlot
	[
		SNew(SScaleBox)
		.Stretch_Lambda([]() -> EStretch::Type { return EStretch::Fill;	})
		[
			SNew(SImage)
			.Image(MaterialBrush.IsValid() ? MaterialBrush.Get() : FAppStyle::GetBrush("WhiteTexture"))
		]
	];
}

void SMediaImage::Tick(const FGeometry& InAllottedGeometry, const double InCurrentTime, const float InDeltaTime)
{
	SCompoundWidget::Tick(InAllottedGeometry, InCurrentTime, InDeltaTime);

	if (MaterialBrush.IsValid() && BrushImageSize.IsSet())
	{
		FVector2D Size = BrushImageSize.Get();
		MaterialBrush->ImageSize.X = static_cast<float>(Size.X);
		MaterialBrush->ImageSize.Y = static_cast<float>(Size.Y);
	}
}

MediaPlayerEditor::MediaImage::ETextureChannelMask SMediaImage::GetChannelMask() const
{
	return Mask;
}

void SMediaImage::SetChannelMask(MediaPlayerEditor::MediaImage::ETextureChannelMask InMask)
{
	using namespace MediaPlayerEditor::MediaImage;

	UTexture* Texture = TextureWeak.Get();

	if (!Texture)
	{
		return;
	}

	// Invalid, no channels specified
	if (!EnumHasAnyFlags(InMask, ETextureChannelMask::RGBA))
	{
		return;
	}

	UMaterial* MaskMaterial = nullptr;

	if (const TStrongObjectPtr<UMaterial>* CachedMaterial = CachedMaterials.Find(InMask))
	{
		MaskMaterial = CachedMaterial->Get();
	}
	else
	{
		switch (InMask)
		{
			case ETextureChannelMask::Red:
			case ETextureChannelMask::Green:
			case ETextureChannelMask::Blue:
			case ETextureChannelMask::Alpha:
				// Create greyscale mask
				MaskMaterial = Private::CreateGreyscaleMaterial(Texture, InMask);
				break;

			default:
				// Create channel mask
				MaskMaterial = Private::CreateChannelMaskMaterial(Texture, InMask);
				break;
		}

		if (!MaskMaterial)
		{
			return;
			
		}
		
		CachedMaterials.Add(InMask, TStrongObjectPtr<UMaterial>(MaskMaterial));
	}

	if (MaskMaterial)
	{
		MaterialAInstance.Reset(Private::CreateInstance(MaskMaterial, Texture));

		if (MaterialBrush.IsValid())
		{
			MaterialBrush->SetResourceObject(MaterialAInstance.Get());
		}

		Mask = InMask;
	}
}
