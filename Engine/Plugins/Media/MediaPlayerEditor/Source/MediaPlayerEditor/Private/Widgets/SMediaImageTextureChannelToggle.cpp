// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/SMediaImageTextureChannelToggle.h"

#include "Framework/Commands/UICommandList.h"
#include "MediaPlayer.h"
#include "Models/MediaPlayerEditorCommands.h"
#include "Styling/AppStyle.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/SMediaPlayerEditorViewer.h"

void SMediaImageTextureChannelToggle::Construct(const FArguments& InArgs, TSharedPtr<SMediaPlayerEditorViewer> InViewer,
	MediaPlayerEditor::MediaImage::ETextureChannelMask InChannel, TSharedPtr<FUICommandList> InCommandList)
{
	ViewerWeak = InViewer;
	Channel = InChannel;
	CommandListWeak = InCommandList;

	using namespace MediaPlayerEditor::MediaImage;

	const FMediaPlayerEditorCommands& Commands = FMediaPlayerEditorCommands::Get();
	TSharedPtr<FUICommandInfo> Command;

	switch (InChannel)
	{
		case ETextureChannelMask::Red:
			Command = Commands.ToggledRedTextureChannel;
			break;

		case ETextureChannelMask::Green:
			Command = Commands.ToggledGreenTextureChannel;
			break;

		case ETextureChannelMask::Blue:
			Command = Commands.ToggledBlueTextureChannel;
			break;

		case ETextureChannelMask::Alpha:
			Command = Commands.ToggledAlphaTextureChannel;
			break;

		default:
			check(false);
			return;
	}

	ChildSlot
	[
		SNew(SBox)
		.Padding(1.f)
		[
			SNew(SCheckBox)
			.Style(FAppStyle::Get(), "TextureEditor.ChannelButtonStyle")
			.BorderBackgroundColor(this, &SMediaImageTextureChannelToggle::GetChannelButtonBackgroundColor)
			.ForegroundColor(this, &SMediaImageTextureChannelToggle::GetChannelButtonForegroundColor)
			.OnCheckStateChanged(this, &SMediaImageTextureChannelToggle::OnChannelButtonCheckStateChanged)
			.IsChecked(this, &SMediaImageTextureChannelToggle::OnGetChannelButtonCheckState)
			.IsEnabled(this, &SMediaImageTextureChannelToggle::IsChannelButtonEnabled)
			.ToolTipText(Command->GetDescription())
			[
				SNew(STextBlock)
				.Font(FAppStyle::Get().GetFontStyle("TextureEditor.ChannelButtonFont"))
				.Text(Command->GetLabel())
			]
		]
	];
}

FSlateColor SMediaImageTextureChannelToggle::GetChannelButtonBackgroundColor() const
{
	using namespace MediaPlayerEditor::MediaImage;

	FSlateColor Dropdown = FAppStyle::Get().GetSlateColor("Colors.Dropdown");

	TSharedPtr<SMediaPlayerEditorViewer> Viewer = ViewerWeak.Pin();

	if (!Viewer.IsValid())
	{
		return FLinearColor::White;
	}

	switch (Channel)
	{
		case ETextureChannelMask::Red:
			return Viewer->IsChannelMasked(Channel) ? FLinearColor::Red : FLinearColor::White;

		case ETextureChannelMask::Green:
			return Viewer->IsChannelMasked(Channel) ? FLinearColor::Green : FLinearColor::White;

		case ETextureChannelMask::Blue:
			return Viewer->IsChannelMasked(Channel) ? FLinearColor::Blue : FLinearColor::White;

		case ETextureChannelMask::Alpha:
			return FLinearColor::White;

		default:
			check(false);
			return FSlateColor();
	}
}

FSlateColor SMediaImageTextureChannelToggle::GetChannelButtonForegroundColor() const
{
	using namespace MediaPlayerEditor::MediaImage;

	FSlateColor DefaultForeground = FAppStyle::Get().GetSlateColor("Colors.Foreground");

	TSharedPtr<SMediaPlayerEditorViewer> Viewer = ViewerWeak.Pin();

	if (!Viewer.IsValid())
	{
		return DefaultForeground;
	}

	switch (Channel)
	{
		case ETextureChannelMask::Red:
			return Viewer->IsChannelMasked(Channel) ? FLinearColor::Black : DefaultForeground;

		case ETextureChannelMask::Green:
			return Viewer->IsChannelMasked(Channel) ? FLinearColor::Black : DefaultForeground;

		case ETextureChannelMask::Blue:
			return Viewer->IsChannelMasked(Channel) ? FLinearColor::Black : DefaultForeground;

		case ETextureChannelMask::Alpha:
			return Viewer->IsChannelMasked(Channel) ? FLinearColor::Black : DefaultForeground;

		default:
			check(false);
			return FSlateColor::UseForeground();
	}
}

ECheckBoxState SMediaImageTextureChannelToggle::OnGetChannelButtonCheckState() const
{
	if (TSharedPtr<SMediaPlayerEditorViewer> Viewer = ViewerWeak.Pin())
	{
		if (Viewer->IsChannelMasked(Channel))
		{
			return ECheckBoxState::Checked;
		}
	}

	return ECheckBoxState::Unchecked;
}

bool SMediaImageTextureChannelToggle::IsChannelButtonEnabled() const
{
	if (TSharedPtr<SMediaPlayerEditorViewer> Viewer = ViewerWeak.Pin())
	{
		if (UMediaPlayer* MediaPlayer = Viewer->GetMediaPlayer())
		{
			return MediaPlayer->IsReady();
		}
	}
	
	return false;
}

void SMediaImageTextureChannelToggle::OnChannelButtonCheckStateChanged(ECheckBoxState InNewState)
{
	using namespace MediaPlayerEditor::MediaImage;

	TSharedPtr<SMediaPlayerEditorViewer> Viewer = ViewerWeak.Pin();

	if (!Viewer.IsValid())
	{
		return;
	}

	TSharedPtr<FUICommandList> CommandList = CommandListWeak.Pin();

	if (!CommandList.IsValid())
	{
		return;
	}

	const FMediaPlayerEditorCommands& Commands = FMediaPlayerEditorCommands::Get();

	switch (Channel)
	{
		case ETextureChannelMask::Red:
			CommandList->TryExecuteAction(Commands.ToggledRedTextureChannel.ToSharedRef());
			break;

		case ETextureChannelMask::Green:
			CommandList->TryExecuteAction(Commands.ToggledGreenTextureChannel.ToSharedRef());
			break;

		case ETextureChannelMask::Blue:
			CommandList->TryExecuteAction(Commands.ToggledBlueTextureChannel.ToSharedRef());
			break;

		case ETextureChannelMask::Alpha:
			CommandList->TryExecuteAction(Commands.ToggledAlphaTextureChannel.ToSharedRef());
			break;

		default:
			check(false);
			break;
	}
}