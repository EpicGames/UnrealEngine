// Copyright Epic Games, Inc. All Rights Reserved.

#include "SMediaPlayerEditorCache.h"

#include "IMediaCache.h"
#include "IMediaPlayer.h"
#include "IMediaSamples.h"
#include "IMediaTracks.h"
#include "MediaPlayer.h"
#include "MediaPlayerFacade.h"
#include "Math/Range.h"
#include "Rendering/DrawElements.h"
#include "Styling/CoreStyle.h"


/* SMediaPlayerEditorOutput structors
 *****************************************************************************/

SMediaPlayerEditorCache::SMediaPlayerEditorCache()
	: MediaPlayer(nullptr)
{ }


/* SMediaPlayerEditorOutput interface
 *****************************************************************************/

void SMediaPlayerEditorCache::Construct(const FArguments& InArgs, UMediaPlayer& InMediaPlayer, const TSharedRef<ISlateStyle>& InStyle)
{
	MediaPlayer = &InMediaPlayer;
	Style = InStyle;

	PositionMarkerMargin = InArgs._PositionMarkerMargin;
	PositionMarkerSize = InArgs._PositionMarkerSize;
	ProgressBarHeight = InArgs._ProgressBarHeight;
}


/* SWidget interface
 *****************************************************************************/

FVector2D SMediaPlayerEditorCache::ComputeDesiredSize(float /*LayoutScaleMultiplier*/) const
{
	return FVector2D(0.0f, ProgressBarHeight.Get() + PositionMarkerMargin.Get() + PositionMarkerSize.Get());
}


int32 SMediaPlayerEditorCache::OnPaint(const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const
{
	// draw background
	FSlateDrawElement::MakeBox(
		OutDrawElements,
		LayerId + 1,
		AllottedGeometry.ToPaintGeometry(FVector2f(AllottedGeometry.Size.X, ProgressBarHeight.Get()), FSlateLayoutTransform()),
		FCoreStyle::Get().GetBrush("GenericWhiteBox"),
		ESlateDrawEffect::None,
		InWidgetStyle.GetColorAndOpacityTint() * FLinearColor::Black
	);

	++LayerId;

	// draw cache state
	if (MediaPlayer->IsReady())
	{
		DrawSampleCache(EMediaTrackType::Audio, AllottedGeometry, OutDrawElements, LayerId, InWidgetStyle, 0.0f, 0.5f);
		DrawSampleCache(EMediaTrackType::Video, AllottedGeometry, OutDrawElements, LayerId, InWidgetStyle, 0.5f, 0.5f);
		DrawPlayerPosition(MediaPlayer->GetDisplayTime(), AllottedGeometry, OutDrawElements, LayerId, InWidgetStyle, FLinearColor::Gray);
	}

	return LayerId + 1;
}


/* SMediaPlayerEditorCache implementation
 *****************************************************************************/

void SMediaPlayerEditorCache::DrawPlayerPosition(FTimespan Time, const FGeometry& AllottedGeometry, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, const FLinearColor& Color) const
{
	static const FSlateBrush* GenericBrush = FCoreStyle::Get().GetBrush("GenericWhiteBox");

	const FTimespan Duration = MediaPlayer->GetDuration();
	const float MarkerSize = PositionMarkerSize.Get();

	const float DrawOffset = static_cast<float>(FTimespan::Ratio(Time, Duration) * AllottedGeometry.Size.X - 0.5f * MarkerSize);

	FSlateDrawElement::MakeBox(
		OutDrawElements,
		LayerId + 1,
		AllottedGeometry.ToPaintGeometry(FVector2f(PositionMarkerSize.Get(), MarkerSize), FSlateLayoutTransform(FVector2f(DrawOffset, ProgressBarHeight.Get() + PositionMarkerMargin.Get()))),
		GenericBrush,
		ESlateDrawEffect::None,
		InWidgetStyle.GetColorAndOpacityTint() * Color
	);
}


void SMediaPlayerEditorCache::DrawSampleCache(EMediaTrackType TrackType, const FGeometry& AllottedGeometry, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, float YPos, float YScale) const
{
	auto PlayerFacade = MediaPlayer->GetPlayerFacade();
	TRangeSet<FTimespan> CacheRangeSet;

	PlayerFacade->QueryCacheState(TrackType, EMediaCacheState::Pending, CacheRangeSet);
	DrawSampleStates(CacheRangeSet, AllottedGeometry, OutDrawElements, LayerId, InWidgetStyle, FLinearColor::Gray, YPos, YScale);

	CacheRangeSet.Empty();

	PlayerFacade->QueryCacheState(TrackType, EMediaCacheState::Loading, CacheRangeSet);
	DrawSampleStates(CacheRangeSet, AllottedGeometry, OutDrawElements, LayerId, InWidgetStyle, FLinearColor::Yellow, YPos, YScale);

	CacheRangeSet.Empty();

	PlayerFacade->QueryCacheState(TrackType, EMediaCacheState::Loaded, CacheRangeSet);
	DrawSampleStates(CacheRangeSet, AllottedGeometry, OutDrawElements, LayerId, InWidgetStyle, FLinearColor(0.10616f, 0.48777f, 0.10616f), YPos, YScale);

	CacheRangeSet.Empty();

	PlayerFacade->QueryCacheState(TrackType, EMediaCacheState::Cached, CacheRangeSet);
	DrawSampleStates(CacheRangeSet, AllottedGeometry, OutDrawElements, LayerId, InWidgetStyle, FLinearColor(0.07059f, 0.32941f, 0.07059f), YPos, YScale);
}


void SMediaPlayerEditorCache::DrawSampleStates(const TRangeSet<FTimespan>& RangeSet, const FGeometry& AllottedGeometry, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, const FLinearColor& Color, float YPos, float YScale) const
{
	static const FSlateBrush* GenericBrush = FCoreStyle::Get().GetBrush("GenericWhiteBox");

	const FTimespan Duration = MediaPlayer->GetDuration();

	FSlateClippingZone ClippingZone(AllottedGeometry);
	OutDrawElements.PushClip(ClippingZone);

	TArray<TRange<FTimespan>> Ranges;
	RangeSet.GetRanges(Ranges);

	for (auto& Range : Ranges)
	{
		const float DrawOffset = static_cast<float>(FMath::RoundToNegativeInfinity(FTimespan::Ratio(Range.GetLowerBoundValue(), Duration) * AllottedGeometry.Size.X));
		const float DrawSize = static_cast<float>(FMath::RoundToPositiveInfinity(FTimespan::Ratio(Range.Size<FTimespan>(), Duration) * AllottedGeometry.Size.X));
		const float BarHeight = ProgressBarHeight.Get();

		FSlateDrawElement::MakeBox(
			OutDrawElements,
			LayerId + 1,
			AllottedGeometry.ToPaintGeometry(FVector2f(DrawSize, YScale * BarHeight), FSlateLayoutTransform(FVector2f(DrawOffset, YPos * BarHeight))),
			GenericBrush,
			ESlateDrawEffect::None,
			InWidgetStyle.GetColorAndOpacityTint() * Color
		);
	}

	OutDrawElements.PopClip();
}
