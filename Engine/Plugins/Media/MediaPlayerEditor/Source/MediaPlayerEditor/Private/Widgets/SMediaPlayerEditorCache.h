// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Math/RangeSet.h"
#include "Misc/Timespan.h"
#include "Styling/ISlateStyle.h"
#include "Templates/SharedPointer.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SLeafWidget.h"

class UMediaPlayer;
enum class EMediaTrackType;


/**
 * Implements the media player cache status visualizer widget.
 */
class SMediaPlayerEditorCache
	: public SLeafWidget
{
public:

	SLATE_BEGIN_ARGS(SMediaPlayerEditorCache)
		: _PositionMarkerMargin(2.0f)
		, _PositionMarkerSize(2.0f)
		, _ProgressBarHeight(10.0f)
	{ }

		/** Top margin of the position marker (in pixels). */
		SLATE_ATTRIBUTE(float, PositionMarkerMargin)

		/** Size of the position marker (in pixels). */
		SLATE_ATTRIBUTE(float, PositionMarkerSize)

		/** Height of the progress bar (in pixels). */
		SLATE_ARGUMENT(float, ProgressBarHeight)

	SLATE_END_ARGS()

	/** Default constructor. */
	SMediaPlayerEditorCache();

public:

	/**
	 * Construct this widget
	 *
	 * @param InArgs The declaration data for this widget.
	 * @param InMediaPlayer The UMediaPlayer asset to show the details for.
	 * @param InStyle The style set to use.
	 */
	void Construct(const FArguments& InArgs, UMediaPlayer& InMediaPlayer, const TSharedRef<ISlateStyle>& InStyle);

public:

	//~ SWidget interface

	virtual FVector2D ComputeDesiredSize(float LayoutScaleMultiplier) const override;
	virtual int32 OnPaint(const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const override;

protected:

	/** Draw the media player's current play head position. */
	void DrawPlayerPosition(FTimespan Time, const FGeometry& AllottedGeometry, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, const FLinearColor& Color) const;

	/** Draw the caching state of the specified track type. */
	void DrawSampleCache(EMediaTrackType TrackType, const FGeometry& AllottedGeometry, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, float YPos, float YScale) const;
	
	/** Draw the caching state of the given media samples. */
	void DrawSampleStates(const TRangeSet<FTimespan>& RangeSet, const FGeometry& AllottedGeometry, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, const FLinearColor& Color, float YPos, float YScale) const;

private:

	/** The media player whose video texture is shown in this widget. */
	UMediaPlayer* MediaPlayer;

	/** Top margin of the position marker (in pixels). */
	TAttribute<float> PositionMarkerMargin;

	/** Size of the position marker (in pixels). */
	TAttribute<float> PositionMarkerSize;

	/** Height of the progress bar (in pixels). */
	TAttribute<float> ProgressBarHeight;

	/** The style set to use for this widget. */
	TSharedPtr<ISlateStyle> Style;
};
