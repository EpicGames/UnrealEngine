// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/SMediaPlayerEditorDetails.h"
#include "Modules/ModuleManager.h"
#include "PropertyEditorModule.h"
#include "IDetailsView.h"
#include "MediaPlayer.h"


#define LOCTEXT_NAMESPACE "SMediaPlayerEditorDetails"


/* SMediaPlayerEditorDetails interface
 *****************************************************************************/

void SMediaPlayerEditorDetails::Construct(const FArguments& InArgs, UMediaPlayer& InMediaPlayer, const TSharedRef<ISlateStyle>& InStyle)
{
	MediaPlayer = &InMediaPlayer;

	// initialize details view
	FDetailsViewArgs DetailsViewArgs;
	{
		DetailsViewArgs.bAllowSearch = true;
		DetailsViewArgs.bHideSelectionTip = true;
		DetailsViewArgs.bLockable = false;
		DetailsViewArgs.bSearchInitialKeyFocus = true;
		DetailsViewArgs.bUpdatesFromSelection = false;
		DetailsViewArgs.bShowOptions = true;
		DetailsViewArgs.bShowModifiedPropertiesOption = false;
	}

	TSharedPtr<IDetailsView> DetailsView = FModuleManager::GetModuleChecked<FPropertyEditorModule>("PropertyEditor").CreateDetailView(DetailsViewArgs);

	DetailsView->SetIsPropertyVisibleDelegate(
		FIsPropertyVisible::CreateLambda(
			[](const FPropertyAndParent& InPropertyAndParent) -> bool
			{
				static const FLazyName HorizontalFieldOfViewName = "HorizontalFieldOfView";
				static const FLazyName VerticalFieldOfViewName = "VerticalFieldOfView";
				static const FLazyName ViewRotationName = "ViewRotation";

				return (InPropertyAndParent.Property.GetFName() != HorizontalFieldOfViewName
					&& InPropertyAndParent.Property.GetFName() != VerticalFieldOfViewName
					&& InPropertyAndParent.Property.GetFName() != ViewRotationName);
			}
		));

	DetailsView->SetObject(MediaPlayer);

	ChildSlot
	[
		DetailsView.ToSharedRef()
	];
}


#undef LOCTEXT_NAMESPACE
