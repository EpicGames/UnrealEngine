// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Styling/ISlateStyle.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"

class UMediaPlayer;

/**
 * Implements the details panel of the MediaPlayer asset editor.
 */
class SMediaPlayerEditorDetails
	: public SCompoundWidget
{
public:

	SLATE_BEGIN_ARGS(SMediaPlayerEditorDetails) { }
	SLATE_END_ARGS()

public:

	/**
	 * Construct this widget
	 *
	 * @param InArgs The declaration data for this widget.
	 * @param InMediaPlayer The MediaPlayer asset to show the details for.
	 * @param InStyleSet The style set to use.
	 */
	void Construct(const FArguments& InArgs, UMediaPlayer& InMediaPlayer, const TSharedRef<ISlateStyle>& InStyle);

private:

	/** Pointer to the MediaPlayer asset that is being viewed. */
	UMediaPlayer* MediaPlayer;
};
