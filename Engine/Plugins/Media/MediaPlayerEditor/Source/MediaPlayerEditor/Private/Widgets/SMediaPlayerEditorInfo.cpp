// Copyright Epic Games, Inc. All Rights Reserved.

#include "SMediaPlayerEditorInfo.h"

#include "Styling/AppStyle.h"
#include "HAL/PlatformApplicationMisc.h"
#include "IMediaControls.h"
#include "MediaPlayer.h"
#include "MediaPlayerFacade.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Layout/SScrollBox.h"
#include "Widgets/Text/STextBlock.h"


#define LOCTEXT_NAMESPACE "SMediaPlayerEditorInfo"


/* Local helpers
 *****************************************************************************/

/**
 * Convert a set of play rates to human readable text.
 *
 * @param Rates The ratest to convert.
 * @return The corresponding text representation.
 */
FText RatesToText(TRangeSet<float> Rates)
{
	static const FNumberFormattingOptions Options = FNumberFormattingOptions()
		.SetMinimumFractionalDigits(1)
		.SetMaximumFractionalDigits(1)
		.SetUseGrouping(false);

	TArray<TRange<float>> OutRanges;
	Rates.GetRanges(OutRanges);

	FString String;

	for (int32 RangeIndex = 0; RangeIndex < OutRanges.Num(); ++RangeIndex)
	{
		if (RangeIndex > 0)
		{
			String += TEXT(", ");
		}

		const TRange<float>& Range = OutRanges[RangeIndex];

		if (Range.IsDegenerate())
		{
			String += FText::Format(LOCTEXT("DegenerateRateFormat", "{0}"), FText::AsNumber(Range.GetLowerBoundValue(), &Options)).ToString();
		}
		else
		{
			String += FText::Format(LOCTEXT("NormalRatesFormat", "{0} to {1}"), FText::AsNumber(Range.GetLowerBoundValue(), &Options), FText::AsNumber(Range.GetUpperBoundValue(), &Options)).ToString();
		}
	}

	return FText::FromString(String);
}


/* SMediaPlayerEditorInfo interface
 *****************************************************************************/

void SMediaPlayerEditorInfo::Construct(const FArguments& InArgs, UMediaPlayer& InMediaPlayer, const TSharedRef<ISlateStyle>& InStyle)
{
	MediaPlayer = &InMediaPlayer;

	ChildSlot
	[
		SNew(SVerticalBox)

		+ SVerticalBox::Slot()
			.FillHeight(1.0f)
			[
				SNew(SScrollBox)

				+ SScrollBox::Slot()
					.Padding(4.0f)
					[
						SAssignNew(InfoTextBlock, STextBlock)
					]
			]

		+ SVerticalBox::Slot()
			.AutoHeight()
			[
				SNew(SBorder)
					.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
					.HAlign(HAlign_Right)
					.Padding(2.0f)
					[
						SNew(SButton)
							.Text(LOCTEXT("CopyClipboardButtonText", "Copy to Clipboard"))
							.ToolTipText(LOCTEXT("CopyClipboardButtonHint", "Copy the media information to the the clipboard"))
							.OnClicked_Lambda([this]() -> FReply {
								FPlatformApplicationMisc::ClipboardCopy(*InfoTextBlock->GetText().ToString());
								return FReply::Handled();
							})
					]
			]
	];

	MediaPlayer->OnMediaEvent().AddSP(this, &SMediaPlayerEditorInfo::HandleMediaPlayerMediaEvent);
}


/* SMediaPlayerEditorInfo callbacks
 *****************************************************************************/

void SMediaPlayerEditorInfo::HandleMediaPlayerMediaEvent(EMediaEvent Event)
{
	static FText NoMediaText = LOCTEXT("NoMediaOpened", "No media opened");

	if ((Event == EMediaEvent::MediaOpened) || (Event == EMediaEvent::MediaOpenFailed) || (Event == EMediaEvent::TracksChanged))
	{
		TSharedRef<FMediaPlayerFacade, ESPMode::ThreadSafe> PlayerFacade = MediaPlayer->GetPlayerFacade();

		if (!PlayerFacade->GetUrl().IsEmpty())
		{
			const FText ThinnedRatesText = RatesToText(PlayerFacade->GetSupportedRates(false));
			const FText UnthinnedRatesText = RatesToText(PlayerFacade->GetSupportedRates(true));

			FFormatNamedArguments Arguments;
			{
				const FCoreTexts& CoreTexts = FCoreTexts::Get();

				Arguments.Add(TEXT("PlayerName"), FText::FromName(PlayerFacade->GetPlayerName()));
				Arguments.Add(TEXT("SupportsScrubbing"), PlayerFacade->CanScrub() ? CoreTexts.Yes : CoreTexts.No);
				Arguments.Add(TEXT("SupportsSeeking"), PlayerFacade->CanSeek() ? CoreTexts.Yes : CoreTexts.No);
				Arguments.Add(TEXT("PlayerInfo"), FText::FromString(PlayerFacade->GetInfo()));

				Arguments.Add(TEXT("ThinnedRates"), ThinnedRatesText.IsEmpty()
					? LOCTEXT("RateNotSupported", "Not supported")
					: ThinnedRatesText
				);

				Arguments.Add(TEXT("UnthinnedRates"), UnthinnedRatesText.IsEmpty()
					? LOCTEXT("RateNotSupported", "Not supported")
					: UnthinnedRatesText
				);
			}

			InfoTextBlock->SetText(
				FText::Format(INVTEXT(
					"Player: {PlayerName}\n"
					"\n"
					"Play Rates\n"
					"    Thinned: {ThinnedRates}\n"
					"    Unthinned: {UnthinnedRates}\n"
					"\n"
					"Capabilities\n"
					"    Scrubbing: {SupportsScrubbing}\n"
					"    Seeking: {SupportsSeeking}\n"
					"\n"
					"{PlayerInfo}"),
					Arguments
				)
			);
		}
		else
		{
			InfoTextBlock->SetText(NoMediaText);
		}
	}
	else if (Event == EMediaEvent::MediaClosed)
	{
		InfoTextBlock->SetText(NoMediaText);
	}
}


#undef LOCTEXT_NAMESPACE
