// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Styling/ISlateStyle.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"

class STextBlock;
class UMediaPlayer;
enum class EMediaEvent;

/**
 * Implements the info panel of the MediaPlayer asset editor.
 */
class SMediaPlayerEditorInfo
	: public SCompoundWidget
{
public:

	SLATE_BEGIN_ARGS(SMediaPlayerEditorInfo) { }
	SLATE_END_ARGS()

public:

	/**
	 * Construct this widget
	 *
	 * @param InArgs The declaration data for this widget.
	 * @param InMediaPlayer The MediaPlayer asset to show the information for.
	 * @param InStyleSet The style set to use.
	 */
	void Construct(const FArguments& InArgs, UMediaPlayer& InMediaPlayer, const TSharedRef<ISlateStyle>& InStyle);

private:

	/** Callback for events from the media player. */
	void HandleMediaPlayerMediaEvent(EMediaEvent Event);

private:

	/** Text block for media information. */
	TSharedPtr<STextBlock> InfoTextBlock;

	/** Pointer to the MediaPlayer asset that is being viewed. */
	UMediaPlayer* MediaPlayer;
};
