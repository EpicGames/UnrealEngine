// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/SMediaPlayerEditorMedia.h"
#include "Modules/ModuleManager.h"
#include "Textures/SlateIcon.h"
#include "IContentBrowserSingleton.h"
#include "ContentBrowserModule.h"
#include "Styling/AppStyle.h"
#include "Editor.h"
#include "Containers/ArrayBuilder.h"

#include "FileMediaSource.h"
#include "MediaPlayer.h"
#include "MediaPlaylist.h"
#include "MediaSource.h"

#include "Framework/Commands/UIAction.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Framework/Notifications/NotificationManager.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Notifications/SNotificationList.h"
#include "Widgets/Text/STextBlock.h"
#include "ToolMenus.h"
#include "Shared/MediaPlayerEditorMediaContext.h"
#include "Subsystems/AssetEditorSubsystem.h"

#define LOCTEXT_NAMESPACE "SMediaPlayerEditorMedia"

const FName SMediaPlayerEditorMedia::AssetPickerAssetContextMenuName = "MediaPlayer.AssetPickerAssetContextMenu";

/* SMediaPlayerEditorMedia interface
 *****************************************************************************/

void SMediaPlayerEditorMedia::Construct(const FArguments& InArgs, UMediaPlayer& InMediaPlayer, const TSharedRef<ISlateStyle>& InStyle)
{
	MediaPlayer = &InMediaPlayer;
	Style = InStyle;

	// initialize asset picker
	FAssetPickerConfig AssetPickerConfig;
	{
		AssetPickerConfig.Filter.ClassPaths.Add(UMediaPlaylist::StaticClass()->GetClassPathName());
		AssetPickerConfig.Filter.ClassPaths.Add(UMediaSource::StaticClass()->GetClassPathName());
		AssetPickerConfig.Filter.bRecursiveClasses = true;

		AssetPickerConfig.AssetShowWarningText = LOCTEXT("NoMediaSourcesFound", "No media sources or play lists found.");
		AssetPickerConfig.bAllowDragging = false;
		AssetPickerConfig.bAutohideSearchBar = true;
		AssetPickerConfig.bCanShowClasses = false;
		AssetPickerConfig.bCanShowDevelopersFolder = true;
		AssetPickerConfig.InitialAssetViewType = EAssetViewType::Column;
		AssetPickerConfig.ThumbnailScale = 0.1f;

		AssetPickerConfig.OnAssetDoubleClicked = FOnAssetDoubleClicked::CreateSP(this, &SMediaPlayerEditorMedia::HandleAssetPickerAssetDoubleClicked);
		AssetPickerConfig.OnAssetEnterPressed = FOnAssetEnterPressed::CreateSP(this, &SMediaPlayerEditorMedia::HandleAssetPickerAssetEnterPressed);
		AssetPickerConfig.OnGetAssetContextMenu = FOnGetAssetContextMenu::CreateSP(this, &SMediaPlayerEditorMedia::HandleAssetPickerGetAssetContextMenu);
	}

	auto& ContentBrowserModule = FModuleManager::Get().LoadModuleChecked<FContentBrowserModule>(TEXT("ContentBrowser"));

	ChildSlot
	[
		SNew(SBorder)
			.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
			.ToolTipText(LOCTEXT("DoubleClickToAddToolTip", "Double-click a media source or playlist to open it in the player."))
			[
				ContentBrowserModule.Get().CreateAssetPicker(AssetPickerConfig)
			]
	];
}


/* SMediaPlayerEditorMedia implementation
 *****************************************************************************/

void SMediaPlayerEditorMedia::OpenMediaAsset(UObject* Asset)
{
	UMediaSource* MediaSource = Cast<UMediaSource>(Asset);

	if (MediaSource != nullptr)
	{
		FMediaPlayerOptions Options;
		Options.SetAllAsOptional();
		Options.InternalCustomOptions.Emplace(MediaPlayerOptionValues::Environment(), MediaPlayerOptionValues::Environment_Preview());
		if (!MediaPlayer->OpenSourceWithOptions(MediaSource, Options))
		{
			ShowMediaOpenFailedMessage();
		}

		return;
	}

	UMediaPlaylist* MediaPlaylist = Cast<UMediaPlaylist>(Asset);

	if (MediaPlaylist != nullptr)
	{
		if (!MediaPlayer->OpenPlaylist(MediaPlaylist))
		{
			ShowMediaOpenFailedMessage();
		}
	}
}


void SMediaPlayerEditorMedia::ShowMediaOpenFailedMessage()
{
	FNotificationInfo NotificationInfo(LOCTEXT("MediaOpenFailedError", "The media failed to open. Check Output Log for details!"));
	{
		NotificationInfo.ExpireDuration = 2.0f;
	}

	FSlateNotificationManager::Get().AddNotification(NotificationInfo)->SetCompletionState(SNotificationItem::CS_Fail);
}


/* SMediaPlayerEditorMedia callbacks
 *****************************************************************************/

void SMediaPlayerEditorMedia::HandleAssetPickerAssetDoubleClicked(const struct FAssetData& AssetData)
{
	OpenMediaAsset(AssetData.GetAsset());
}


void SMediaPlayerEditorMedia::HandleAssetPickerAssetEnterPressed(const TArray<FAssetData>& SelectedAssets)
{
	if (SelectedAssets.Num() > 0)
	{
		OpenMediaAsset(SelectedAssets[0].GetAsset());
	}
}

void SMediaPlayerEditorMedia::RegisterMenus()
{
	UToolMenu* Menu = UToolMenus::Get()->RegisterMenu(AssetPickerAssetContextMenuName);
	{
		FToolMenuSection& Section = Menu->AddSection("MediaSection", LOCTEXT("MediaSection", "Media"));
		Section.AddEntry(FToolMenuEntry::InitMenuEntry(
			"Edit",
			LOCTEXT("EditMenuAction", "Edit..."),
			LOCTEXT("EditMenuActionTooltip", "Opens the selected asset for edit."),
			FSlateIcon(FAppStyle::GetAppStyleSetName(), "ContentBrowser.AssetActions.Edit"),
			FToolMenuExecuteAction::CreateLambda([](const FToolMenuContext& InContext)
			{
				if (UMediaPlayerEditorMediaContext* Context = InContext.FindContext<UMediaPlayerEditorMediaContext>())
				{
					if (Context->SelectedAsset)
					{
						GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->OpenEditorForAsset(Context->SelectedAsset);
					}
				}
			})
		));


		Section.AddDynamicEntry("Open", FNewToolMenuSectionDelegate::CreateLambda([](FToolMenuSection& InSection)
		{
			if (UMediaPlayerEditorMediaContext* Context = InSection.FindContext<UMediaPlayerEditorMediaContext>())
			{
				InSection.AddEntry(FToolMenuEntry::InitMenuEntry(
					"Open",
					LOCTEXT("OpenMenuAction", "Open"),
					LOCTEXT("OpenMenuActionTooltip", "Open this media asset in the player"),
					FSlateIcon(Context->StyleSetName, "MediaPlayerEditor.NextMedia.Small"),
					FToolMenuExecuteAction::CreateLambda([=](const FToolMenuContext& InContext)
					{
						if (UMediaPlayerEditorMediaContext* Context = InContext.FindContext<UMediaPlayerEditorMediaContext>())
						{
							if (Context->MediaPlayerEditorMedia.IsValid())
							{
								Context->MediaPlayerEditorMedia.Pin()->OpenMediaAsset(Context->SelectedAsset);
							}
						}
					})
				));
			}
		}));
	}

	{
		FToolMenuSection& Section = Menu->AddSection("AssetSection", LOCTEXT("AssetSection", "Asset"));
		Section.AddEntry(FToolMenuEntry::InitMenuEntry(
			"FindInCbMenuAction",
			LOCTEXT("FindInCbMenuAction", "Find in Content Browser"),
			LOCTEXT("FindInCbMenuActionTooltip", "Summons the Content Browser and navigates to the selected asset"),
			FSlateIcon(FAppStyle::GetAppStyleSetName(), "SystemWideCommands.FindInContentBrowser"),
			FToolMenuExecuteAction::CreateLambda([](const FToolMenuContext& InContext)
			{
				if (UMediaPlayerEditorMediaContext* Context = InContext.FindContext<UMediaPlayerEditorMediaContext>())
				{
					TArray<UObject*> AssetsToSync = TArrayBuilder<UObject*>().Add(Context->SelectedAsset);
					GEditor->SyncBrowserToObjects(AssetsToSync);
				}
			})
		));

		Section.AddDynamicEntry("OpenInFileManager", FNewToolMenuSectionDelegate::CreateLambda([](FToolMenuSection& InSection)
		{
			if (UMediaPlayerEditorMediaContext* Context = InSection.FindContext<UMediaPlayerEditorMediaContext>())
			{
				if (UFileMediaSource* FileMediaSource = Cast<UFileMediaSource>(Context->SelectedAsset))
				{
					FFormatNamedArguments Args;
					{
						Args.Add(TEXT("FileManagerName"), FPlatformMisc::GetFileManagerName());
					}

					TWeakObjectPtr<UFileMediaSource> FileMediaSourceWeak = FileMediaSource;
					InSection.AddEntry(FToolMenuEntry::InitMenuEntry(
						"OpenInFileManager",
						FText::Format(LOCTEXT("OpenInFileManager", "Show Media File in {FileManagerName}"), Args),
						LOCTEXT("OpenInFileManagerTooltip", "Finds the media file that this asset points to on disk"),
						FSlateIcon(FAppStyle::GetAppStyleSetName(), "SystemWideCommands.FindInContentBrowser"),
						FUIAction(
							FExecuteAction::CreateLambda([=]() {
								if (FileMediaSourceWeak.IsValid())
								{
									FPlatformProcess::ExploreFolder(*FileMediaSourceWeak->GetFullPath());
								}
							}),
							FCanExecuteAction::CreateLambda([=]() -> bool {
								return FileMediaSourceWeak.IsValid() && FileMediaSourceWeak->Validate();
							})
						)
					));
				}
			}
		}));
	}
}

TSharedPtr<SWidget> SMediaPlayerEditorMedia::HandleAssetPickerGetAssetContextMenu(const TArray<FAssetData>& SelectedAssets)
{
	if (SelectedAssets.Num() <= 0)
	{
		return nullptr;
	}

	UObject* SelectedAsset = SelectedAssets[0].GetAsset();
	if (SelectedAsset == nullptr)
	{
		return nullptr;
	}

	UMediaPlayerEditorMediaContext* ContextObject = NewObject<UMediaPlayerEditorMediaContext>();
	ContextObject->InitContext(SelectedAsset, Style->GetStyleSetName());
	ContextObject->MediaPlayerEditorMedia = SharedThis(this);
	FToolMenuContext MenuContext(ContextObject);
	return UToolMenus::Get()->GenerateWidget(AssetPickerAssetContextMenuName, MenuContext);
}


#undef LOCTEXT_NAMESPACE
