// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "AssetRegistry/AssetData.h"
#include "Styling/ISlateStyle.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"

class SWidget;
class UMediaPlayer;


/**
 * Implements the media library of the MediaPlayer asset editor.
 */
class SMediaPlayerEditorMedia
	: public SCompoundWidget
{
public:

	SLATE_BEGIN_ARGS(SMediaPlayerEditorMedia) { }
	SLATE_END_ARGS()

public:

	/**
	 * Construct this widget
	 *
	 * @param InArgs The declaration data for this widget.
	 * @param InMediaPlayer The MediaPlayer asset to show the details for.
	 * @param InStyleSet The style set to use.
	 */
	void Construct(const FArguments& InArgs, UMediaPlayer& InMediaPlayer, const TSharedRef<ISlateStyle>& InStyle);

	/* Register menus associated with this widget */
	static void RegisterMenus();

	/**
	 * Open an asset in the media player.
	 *
	 * @param Asset The asset to open.
	 */
	void OpenMediaAsset(UObject* Asset);

protected:
	/** Show a message box for media opening failures. */
	static void ShowMediaOpenFailedMessage();

private:

	/** Callback for double-clicking an asset in the asset picker. */
	void HandleAssetPickerAssetDoubleClicked(const struct FAssetData& AssetData);

	/** Callback for pressing Enter on a selected asset in the asset picker. */
	void HandleAssetPickerAssetEnterPressed(const TArray<FAssetData>& SelectedAssets);

	/** Callback for getting the context menu of an asset in the asset picker. */
	TSharedPtr<SWidget> HandleAssetPickerGetAssetContextMenu(const TArray<FAssetData>& SelectedAssets);

private:

	/** Pointer to the MediaPlayer asset that is being viewed. */
	UMediaPlayer* MediaPlayer;

	/** The widget style set to use. */
	TSharedPtr<ISlateStyle> Style;

	/** Name of the asset picker asset context menu */
	static const FName AssetPickerAssetContextMenuName;
};
