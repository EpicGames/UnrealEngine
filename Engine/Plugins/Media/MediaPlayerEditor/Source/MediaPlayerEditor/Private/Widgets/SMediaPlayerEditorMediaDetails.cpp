// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/SMediaPlayerEditorMediaDetails.h"
#include "IDetailsView.h"
#include "MediaPlayer.h"
#include "MediaTexture.h"
#include "Modules/ModuleManager.h"
#include "PropertyEditorModule.h"
#include "Widgets/Layout/SScrollBox.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "SMediaPlayerEditorMediaDetails"

/* SMediaPlayerEditorMediaDetails interface
 *****************************************************************************/

void SMediaPlayerEditorMediaDetails::Construct(const FArguments& InArgs,
	UMediaPlayer* InMediaPlayer, UMediaTexture* InMediaTexture, FName InTextStyleName)
{
	// Remark: these can be null.
	MediaPlayerWeak = InMediaPlayer;
	MediaTextureWeak = InMediaTexture;

	const ISlateStyle& SlateStyle = FAppStyle::Get();
	const FName StyleName = InTextStyleName.IsNone() ? TEXT("NormalText") : InTextStyleName;

	ChildSlot
		[
			SNew(SScrollBox)

			// Add details.
			+ SScrollBox::Slot()
				[
					SNew(SHorizontalBox)

					// Left side.
					+ SHorizontalBox::Slot()
						[
							SNew(SVerticalBox)

							// Player name.
							+ SVerticalBox::Slot()
								.AutoHeight()
								.VAlign(VAlign_Center)
								.Padding(4.0f)
								[
									SAssignNew(MediaPlayerName, STextBlock)
									.TextStyle(SlateStyle, StyleName)
								]

							// Resolution.
							+ SVerticalBox::Slot()
								.AutoHeight()
								.VAlign(VAlign_Center)
								.Padding(4.0f)
								[
									SAssignNew(ResolutionText, STextBlock)
									.TextStyle(SlateStyle, StyleName)
								]

							// Frame rate.
							+ SVerticalBox::Slot()
								.AutoHeight()
								.VAlign(VAlign_Center)
								.Padding(4.0f)
								[
									SAssignNew(FrameRateText, STextBlock)
									.TextStyle(SlateStyle, StyleName)
								]

							// Resource size.
							+ SVerticalBox::Slot()
								.AutoHeight()
								.VAlign(VAlign_Center)
								.Padding(4.0f)
								[
									SAssignNew(ResourceSizeText, STextBlock)
									.TextStyle(SlateStyle, StyleName)
								]

							// Method.
							+ SVerticalBox::Slot()
								.AutoHeight()
								.VAlign(VAlign_Center)
								.Padding(4.0f)
								[
									SAssignNew(MethodText, STextBlock)
									.TextStyle(SlateStyle, StyleName)
								]
						]

					// Right side.
					+ SHorizontalBox::Slot()
						[
							SNew(SVerticalBox)

							// LOD bias.
							+ SVerticalBox::Slot()
								.AutoHeight()
								.VAlign(VAlign_Center)
								.Padding(4.0f)
								[
									SAssignNew(LODBiasText, STextBlock)
									.TextStyle(SlateStyle, StyleName)
								]

							// Num mips.
							+ SVerticalBox::Slot()
								.AutoHeight()
								.VAlign(VAlign_Center)
								.Padding(4.0f)
								[
									SAssignNew(NumMipsText, STextBlock)
									.TextStyle(SlateStyle, StyleName)
								]

							// Num tiles.
							+ SVerticalBox::Slot()
								.AutoHeight()
								.VAlign(VAlign_Center)
								.Padding(4.0f)
								[
									SAssignNew(NumTilesText, STextBlock)
									.TextStyle(SlateStyle, StyleName)
								]

							// Format.
							+ SVerticalBox::Slot()
								.AutoHeight()
								.VAlign(VAlign_Center)
								.Padding(4.0f)
								[
									SAssignNew(FormatText, STextBlock)
									.TextStyle(SlateStyle, StyleName)
								]

							// Start Timecode.
							+ SVerticalBox::Slot()
								.AutoHeight()
								.VAlign(VAlign_Center)
								.Padding(4.0f)
								[
									SAssignNew(StartTimecodeText, STextBlock)
									.TextStyle(SlateStyle, StyleName)
								]

							// Seek Performance
							+ SVerticalBox::Slot()
								.AutoHeight()
								.VAlign(VAlign_Center)
								.Padding(4.0f)
								[
									SAssignNew(SeekPerformance, STextBlock)
									.TextStyle(SlateStyle, StyleName)
								]

							// Video decoder provider
							+ SVerticalBox::Slot()
								.AutoHeight()
								.VAlign(VAlign_Center)
								.Padding(4.0f)
								[
									SAssignNew(VideoDecoderProvider, STextBlock)
									.TextStyle(SlateStyle, StyleName)
								]

							// Audio decoder provider
							+ SVerticalBox::Slot()
								.AutoHeight()
								.VAlign(VAlign_Center)
								.Padding(4.0f)
								[
									SAssignNew(AudioDecoderProvider, STextBlock)
									.TextStyle(SlateStyle, StyleName)
								]
						]
				]

		];

	UpdateDetails();
}


void SMediaPlayerEditorMediaDetails::Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime)
{
	// Call parent.
	SCompoundWidget::Tick(AllottedGeometry, InCurrentTime, InDeltaTime);

	UpdateDetails();
}

UMediaPlayer* SMediaPlayerEditorMediaDetails::GetMediaPlayer() const
{
	return MediaPlayerWeak.Get();
}

UMediaTexture* SMediaPlayerEditorMediaDetails::GetMediaTexture() const
{
	return MediaTextureWeak.Get();
}

void SMediaPlayerEditorMediaDetails::UpdateDetails()
{
	FName PlayerName;
	FString Format;
	float FrameRate = 0.0f;
	int32 LODBias = 0;
	FText Method;
	int32 NumMips = 0;
	int32 NumTotalTiles = 0;
	int64 ResourceSize = 0;
	int32 SurfaceWidth = 0;
	int32 SurfaceHeight = 0;
	FString StartTimecode;
	int32 KeyframeInterval = -1;
	FString VideoDecoderImpl;
	FString AudioDecoderImpl;

	const UMediaPlayer* MediaPlayer = GetMediaPlayer();
	UMediaTexture* MediaTexture = GetMediaTexture();
	
	// Get player info.
	if (MediaPlayer != nullptr)
	{
		PlayerName = MediaPlayer->GetPlayerName();
		FrameRate = MediaPlayer->GetVideoTrackFrameRate(INDEX_NONE, INDEX_NONE);
		Format = MediaPlayer->GetVideoTrackType(INDEX_NONE, INDEX_NONE);
		FIntPoint NumTiles(EForceInit::ForceInitToZero);
		MediaPlayer->GetMediaInfo<FIntPoint>(NumTiles, UMediaPlayer::MediaInfoNameSourceNumTiles.Resolve());
		NumTotalTiles = NumTiles.X * NumTiles.Y;
		MediaPlayer->GetMediaInfo<FString>(StartTimecode, UMediaPlayer::MediaInfoNameStartTimecodeValue.Resolve());
		MediaPlayer->GetMediaInfo<int32>(KeyframeInterval, UMediaPlayer::MediaInfoNameKeyframeInterval.Resolve());
		MediaPlayer->GetMediaInfo<FString>(VideoDecoderImpl, UMediaPlayer::MediaInfoNameVideoDecoderProvider.Resolve());
		MediaPlayer->GetMediaInfo<FString>(AudioDecoderImpl, UMediaPlayer::MediaInfoNameAudioDecoderProvider.Resolve());
	}

	// Get texture info,
	if (MediaTexture != nullptr)
	{
		LODBias = MediaTexture->GetCachedLODBias();
		Method = MediaTexture->IsCurrentlyVirtualTextured() ?
			LOCTEXT("MethodVirtualStreamed", "Virtual Streamed")
			: (!MediaTexture->IsStreamable() ? LOCTEXT("QuickInfo_MethodNotStreamed", "Not Streamed")
				: LOCTEXT("MethodStreamed", "Streamed"));
		NumMips = MediaTexture->GetTextureNumMips();
		ResourceSize = (MediaTexture->GetResourceSizeBytes(EResourceSizeMode::Exclusive) + 512) / 1024;
		SurfaceWidth = static_cast<int32>(MediaTexture->GetSurfaceWidth());
		SurfaceHeight = static_cast<int32>(MediaTexture->GetSurfaceHeight());
	}

	// Update text.
	MediaPlayerName->SetText(FText::Format(LOCTEXT("Player", "Player: {0}"),
		FText::FromName(PlayerName)));
	FormatText->SetText(FText::Format(LOCTEXT("Format", "Format: {0}"),
		FText::FromString(Format)));
	FrameRateText->SetText(FText::Format(LOCTEXT("FrameRate", "Frame Rate: {0}"),
		FText::AsNumber(FrameRate)));
	LODBiasText->SetText(FText::Format(LOCTEXT("LODBias", "Combined LOD Bias: {0}"),
		FText::AsNumber(LODBias)));
	MethodText->SetText(FText::Format(LOCTEXT("Method", "Method: {0}"), Method));
	NumMipsText->SetText(FText::Format(LOCTEXT("NumberOfMips", "Mips Qty: {0}"),
		FText::AsNumber(NumMips)));
	NumTilesText->SetText(FText::Format(LOCTEXT("NumberOfTiles", "Tiles Qty: {0}"),
		FText::AsNumber(NumTotalTiles)));
	ResolutionText->SetText(FText::Format(LOCTEXT("Resolution", "Resolution: {0}x{1}"),
		FText::AsNumber(SurfaceWidth), FText::AsNumber(SurfaceHeight)));
	ResourceSizeText->SetText(FText::Format(LOCTEXT("ResourceSize", "Resource Size: {0} KB"),
		FText::AsNumber(ResourceSize)));

	if (!StartTimecode.IsEmpty())
	{
		StartTimecodeText->SetText(FText::Format(LOCTEXT("StartTimecode", "Start Timecode: {0}"), FText::FromString(StartTimecode)));
	}
	StartTimecodeText->SetVisibility(!StartTimecode.IsEmpty() ? EVisibility::Visible : EVisibility::Collapsed);

	/*
		The keyframe interval is one of:
		  -1 : no information returned (unknown)
		   0 : unknown keyframe spacing, not every frame is a keyframe but the spacing is variable or cannot be determined
		   1 : every frame is a keyframe
		  >1 : every n'th frame is a keyframe
	*/
	if (KeyframeInterval >= 0)
	{
		SeekPerformance->SetText(FText::Format(LOCTEXT("SeekPerformance", "Seek Performance: {0}"), KeyframeInterval == 1 ? LOCTEXT("SeekPerformanceF", "Fast") : LOCTEXT("SeekPerformanceS", "Slow (GOP codec)")));
		SeekPerformance->SetVisibility(EVisibility::Visible);
	}
	else
	{
		SeekPerformance->SetVisibility(EVisibility::Collapsed);
	}

	if (!VideoDecoderImpl.IsEmpty())
	{
		VideoDecoderProvider->SetText(FText::Format(LOCTEXT("VideoDecoderProvider", "Video decoder: {0}"), FText::FromString(VideoDecoderImpl)));
		VideoDecoderProvider->SetVisibility(EVisibility::Visible);
	}
	else
	{
		VideoDecoderProvider->SetVisibility(EVisibility::Collapsed);
	}
	if (!AudioDecoderImpl.IsEmpty())
	{
		AudioDecoderProvider->SetText(FText::Format(LOCTEXT("AudioDecoderProvider", "Audio decoder: {0}"), FText::FromString(AudioDecoderImpl)));
		AudioDecoderProvider->SetVisibility(EVisibility::Visible);
	}
	else
	{
		AudioDecoderProvider->SetVisibility(EVisibility::Collapsed);
	}
}

#undef LOCTEXT_NAMESPACE
