// Copyright Epic Games, Inc. All Rights Reserved.

#include "SMediaPlayerEditorOutput.h"

#include "Editor.h"
#include "Engine/Engine.h"
#include "IMediaEventSink.h"
#include "Materials/Material.h"
#include "MediaPlayer.h"
#include "MediaSoundComponent.h"
#include "MediaTexture.h"
#include "MediaTextureTracker.h"
#include "Providers/ViewportTileVisibilityProvider.h"
#include "Widgets/Images/SImage.h"

/* SMediaPlayerEditorOutput structors
 *****************************************************************************/

SMediaPlayerEditorOutput::SMediaPlayerEditorOutput()
	: MediaPlayer(nullptr)
	, MediaTexture(nullptr)
	, SoundComponent(nullptr)
	, bIsOurMediaTexture(false)
{ }


SMediaPlayerEditorOutput::~SMediaPlayerEditorOutput()
{
	if (TextureTrackerObject.IsValid())
	{
		if (MediaTexture != nullptr)
		{
			FMediaTextureTracker::Get().UnregisterTexture(TextureTrackerObject, MediaTexture);
		}
		TextureTrackerObject->TileVisibilityProvider.Reset();
		TextureTrackerObject.Reset();
	}
	TileVisibilityProvider.Reset();

	if (MediaPlayer.IsValid())
	{
		MediaPlayer->OnMediaEvent().RemoveAll(this);
		MediaPlayer.Reset();
	}

	// Did we create the media texture?
	if (bIsOurMediaTexture)
	{
		bIsOurMediaTexture = false;
		if (MediaTexture != nullptr)
		{
			MediaTexture->RemoveFromRoot();
			MediaTexture = nullptr;
		}
	}

	if (SoundComponent != nullptr)
	{
		SoundComponent->Stop();
		SoundComponent->RemoveFromRoot();
		SoundComponent->SetMediaPlayer(nullptr);
		SoundComponent->UpdatePlayer();
		SoundComponent = nullptr;
	}
}


/* SMediaPlayerEditorOutput interface
 *****************************************************************************/

void SMediaPlayerEditorOutput::Construct(const FArguments& InArgs, UMediaPlayer& InMediaPlayer,
	UMediaTexture* InMediaTexture, bool bInIsSoundEnabled)
{
	MediaPlayer = &InMediaPlayer;

	// create media sound component
	if ((GEngine != nullptr) && GEngine->UseSound() && (bInIsSoundEnabled))
	{
		SoundComponent = NewObject<UMediaSoundComponent>(GetTransientPackage(), NAME_None, RF_Transient | RF_Public);

		if (SoundComponent != nullptr)
		{
			SoundComponent->bIsUISound = true;
			SoundComponent->bIsPreviewSound = true;
			SoundComponent->SetMediaPlayer(&InMediaPlayer);
			SoundComponent->Initialize();
			SoundComponent->AddToRoot();
		}
	}

	// Did we get media texture passed in?
	if (InMediaTexture != nullptr)
	{
		MediaTexture = InMediaTexture;
		bIsOurMediaTexture = false;
	}
	else
	{
		// create media texture
		MediaTexture = NewObject<UMediaTexture>(GetTransientPackage(), NAME_None, RF_Transient | RF_Public);
		bIsOurMediaTexture = true;

		if (MediaTexture != nullptr)
		{
			MediaTexture->AutoClear = true;
			MediaTexture->SetMediaPlayer(&InMediaPlayer);
			MediaTexture->SetColorSpaceOverride(UE::Color::EColorSpace::sRGB);
			MediaTexture->UpdateResource();
			MediaTexture->AddToRoot();
		}
	}

	SAssignNew(MediaImage, SMediaImage, MediaTexture)
		.BrushImageSize_Lambda([&]()
			{
				if (MediaTexture)
					return FVector2D(MediaTexture->GetSurfaceWidth(), MediaTexture->GetSurfaceHeight());
				else
					return FVector2D::ZeroVector;
			});

	ChildSlot
	[
		MediaImage.ToSharedRef()
	];

	// Hand the on-screen viewport size to tiled-media readers via FMediaTextureTracker so they
	// can pick the right mip and avoid pulling tiles the user never sees.
	if (MediaTexture != nullptr)
	{
		TileVisibilityProvider = MakeShared<UE::MediaPlayerEditor::FViewportTileVisibilityProvider, ESPMode::ThreadSafe>();

		TextureTrackerObject = MakeShared<FMediaTextureTrackerObject, ESPMode::ThreadSafe>();
		TextureTrackerObject->TileVisibilityProvider = TileVisibilityProvider;

		FMediaTextureTracker::Get().RegisterTexture(TextureTrackerObject, MediaTexture);
	}

	MediaPlayer->OnMediaEvent().AddRaw(this, &SMediaPlayerEditorOutput::HandleMediaPlayerMediaEvent);
}

MediaPlayerEditor::MediaImage::ETextureChannelMask SMediaPlayerEditorOutput::GetChannelMask() const
{
	if (MediaImage.IsValid())
	{
		return MediaImage->GetChannelMask();
	}

	return MediaPlayerEditor::MediaImage::ETextureChannelMask::RGBA;
}

void SMediaPlayerEditorOutput::SetChannelMask(MediaPlayerEditor::MediaImage::ETextureChannelMask InMask)
{
	if (MediaImage.IsValid())
	{
		MediaImage->SetChannelMask(InMask);
	}
}


/* SWidget interface
 *****************************************************************************/

void SMediaPlayerEditorOutput::Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime)
{
	if (SoundComponent != nullptr)
	{
		SoundComponent->UpdatePlayer();
	}

	if (TileVisibilityProvider.IsValid())
	{
		// Prefer the child SMediaImage's actual rendered geometry once it's been laid out -
		// it reflects aspect-ratio adjustments the parent slot doesn't. Falls back to our
		// own allotted geometry on the first tick before the child has a cached geometry.
		FVector2D AbsSize = AllottedGeometry.GetAbsoluteSize();
		if (MediaImage.IsValid())
		{
			const FVector2D ChildSize = MediaImage->GetTickSpaceGeometry().GetAbsoluteSize();
			if (ChildSize.X > 0.0 && ChildSize.Y > 0.0)
			{
				AbsSize = ChildSize;
			}
		}
		TileVisibilityProvider->SetDisplaySizePx(FIntPoint(
			FMath::CeilToInt32(AbsSize.X),
			FMath::CeilToInt32(AbsSize.Y)));
	}
}


/* SMediaPlayerEditorOutput callbacks
 *****************************************************************************/

void SMediaPlayerEditorOutput::HandleMediaPlayerMediaEvent(EMediaEvent Event)
{
	if (SoundComponent == nullptr)
	{
		return;
	}

	if (Event == EMediaEvent::PlaybackSuspended)
	{
		SoundComponent->Stop();
	}
	else if (Event == EMediaEvent::PlaybackResumed)
	{
		if (GEditor->PlayWorld == nullptr)
		{
			SoundComponent->Start();
		}
	}
}
