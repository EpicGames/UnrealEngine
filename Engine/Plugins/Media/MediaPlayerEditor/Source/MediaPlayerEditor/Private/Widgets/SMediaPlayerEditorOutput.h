// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Templates/SharedPointer.h"
#include "UObject/WeakObjectPtr.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/SMediaImage.h"

class UMediaPlayer;
class UMediaSoundComponent;
class UMediaTexture;

enum class EMediaEvent;

struct FMediaTextureTrackerObject;

namespace UE::MediaPlayerEditor
{
	class FViewportTileVisibilityProvider;
}


/**
 * Handles content output in the viewer tab in the UMediaPlayer asset editor.
 */
class SMediaPlayerEditorOutput
	: public SCompoundWidget
{
public:

	SLATE_BEGIN_ARGS(SMediaPlayerEditorOutput) { }
	SLATE_END_ARGS()

public:

	/** Default constructor. */
	SMediaPlayerEditorOutput();

	/** Destructor. */
	~SMediaPlayerEditorOutput();

public:

	/**
	 * Construct this widget
	 *
	 * @param InArgs The declaration data for this widget.
	 * @param InMediaPlayer The UMediaPlayer asset to show the details for.
	 * @param InMediaTexture The UMediaTexture asset to output video to. If nullptr then use our own.
	 * @param bInIsSoundEnabled If true then produce sound.
	 */
	void Construct(const FArguments& InArgs, UMediaPlayer& InMediaPlayer,
		UMediaTexture* InMediaTexture, bool  bInIsSoundEnabled);

	/**
	 * Returns the active mask.
	 */
	MediaPlayerEditor::MediaImage::ETextureChannelMask GetChannelMask() const;

	/**
	 * Sets a channel-based mask for the image.
	 *
	 * If only a single channel is displayed, it will show in greyscale.
	 *
	 * @param InMask to display.
	 */
	void SetChannelMask(MediaPlayerEditor::MediaImage::ETextureChannelMask InMask);

public:

	//~ SWidget interface

	virtual void Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime) override;

private:

	/** Callback for media events from the media player. */
	void HandleMediaPlayerMediaEvent(EMediaEvent Event);

private:

	/** The media player whose video texture is shown in this widget. */
	TWeakObjectPtr<UMediaPlayer> MediaPlayer;

	/** The media texture to render the media player's video output. */
	UMediaTexture* MediaTexture;

	/** The sound component to play the media player's audio output. */
	UMediaSoundComponent* SoundComponent;

	/** If true then we created MediaTexture and need to clean it up when we are done. */
	bool bIsOurMediaTexture;

	/** The image widget displaying the media. */
	TSharedPtr<SMediaImage> MediaImage;

	/**
	 * Tile-visibility provider that tells tiled-media readers (e.g. ImgMedia EXR sequences)
	 * to load only the tiles needed for this widget's on-screen pixel area. Owned here, held
	 * weakly by the per-loader resolver. Reset in the destructor before unregistering.
	 */
	TSharedPtr<UE::MediaPlayerEditor::FViewportTileVisibilityProvider, ESPMode::ThreadSafe> TileVisibilityProvider;

	/** FMediaTextureTracker bookkeeping for our provider. Registered against MediaTexture. */
	TSharedPtr<FMediaTextureTrackerObject, ESPMode::ThreadSafe> TextureTrackerObject;
};
