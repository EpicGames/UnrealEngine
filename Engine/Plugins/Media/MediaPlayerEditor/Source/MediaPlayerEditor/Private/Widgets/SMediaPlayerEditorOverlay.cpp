// Copyright Epic Games, Inc. All Rights Reserved.

#include "SMediaPlayerEditorOverlay.h"

#include "IMediaOverlaySample.h"
#include "IMediaEventSink.h"
#include "MediaPlayer.h"
#include "MediaPlayerFacade.h"
#include "MediaPlayerEditorLog.h"
#include "MediaSampleQueue.h"
#include "Widgets/Layout/SConstraintCanvas.h"
#include "Widgets/Text/SRichTextBlock.h"
#include "Styling/SlateStyleMacros.h"

struct SMediaPlayerEditorOverlay::FConfig
{
	FTextBlockStyle TextStyle;
	FLinearColor BackgroundColor;
	FSlateBrush BackgroundBrush;
};


class SMediaPlayerEditorOverlay::FInternal : public TSharedFromThis<SMediaPlayerEditorOverlay::FInternal, ESPMode::ThreadSafe>
{
public:
	FInternal()
	{
	}
	~FInternal()
	{
	}
	void OnSinkEvent(EMediaSampleSinkEvent InEvent, const FMediaSampleSinkEventData& InData)
	{
		switch(InEvent)
		{
			case EMediaSampleSinkEvent::Attached:
			case EMediaSampleSinkEvent::Detached:
			case EMediaSampleSinkEvent::FlushWasRequested:
			case EMediaSampleSinkEvent::MediaClosed:
			case EMediaSampleSinkEvent::PlaybackEndReached:
			{
				PrevPlayerTime = FTimespan::MinValue();
				ClearAfterPlayerTime = FTimespan::MaxValue();
				bIsDirty = true;
				break;
			}
			default:
			{
				break;
			}
		}
	}
	TSharedPtr<FMediaOverlaySampleQueue, ESPMode::ThreadSafe> SubtitleQueue;
	FTimespan PrevPlayerTime { FTimespan::MinValue() };
	FTimespan ClearAfterPlayerTime { FTimespan::MaxValue() };
	int32 SelectedTrack = -1;
	bool bIsDirty = true;
};


/* SMediaPlayerEditorOverlay interface
 *****************************************************************************/

void SMediaPlayerEditorOverlay::Construct(const FArguments& InArgs, UMediaPlayer& InMediaPlayer)
{
	MediaPlayer = &InMediaPlayer;

	// Create a configuration for subtitle display.
	Configuration = MakePimpl<FConfig>();
	Configuration->TextStyle = FTextBlockStyle()
		.SetFont(DEFAULT_FONT("Regular", 20))
		.SetColorAndOpacity(FSlateColor::UseForeground())
//		.SetShadowOffset(FVector2D(3.0))
//		.SetShadowColorAndOpacity(FLinearColor::Black)
		;
	Configuration->TextStyle.Font.OutlineSettings.OutlineSize = 2;
	Configuration->BackgroundColor = FLinearColor { 0.5f, 0.5f, 0.5f, 1.0f };
	Configuration->BackgroundBrush.TintColor = Configuration->BackgroundColor;

	ChildSlot
	[
		SAssignNew(Canvas, SConstraintCanvas)
	];

	// Create a sample queue for the subtitles and add it as a sampel sink.
	Internal = MakeShared<FInternal, ESPMode::ThreadSafe>();
	Internal->SubtitleQueue = MakeShared<FMediaOverlaySampleQueue, ESPMode::ThreadSafe>();
	Internal->SubtitleQueue->OnMediaSampleSinkEvent().AddThreadSafeSP(Internal->AsShared(), &FInternal::OnSinkEvent);
	MediaPlayer->GetPlayerFacade()->AddSubtitleSampleSink(Internal->SubtitleQueue.ToSharedRef());
}


/* SWidget interface
 *****************************************************************************/

void SMediaPlayerEditorOverlay::Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime)
{
	// Check if a new subtitle sample has arrived.
	if (!Internal.IsValid() || !Internal->SubtitleQueue.IsValid())
	{
		Canvas->ClearChildren();
		return;
	}

	if (MediaPlayer)
	{
		// Check if the selected subtitle track has changed.
		int32 CurrentTrack = MediaPlayer->GetSelectedTrack(EMediaPlayerTrack::Subtitle);
		if (CurrentTrack != Internal->SelectedTrack)
		{
			Internal->SelectedTrack = CurrentTrack;
			Internal->bIsDirty = true;
		}
		// Check if the playback time wrapped around.
		const float CurrentRate = MediaPlayer->GetRate();
		if (CurrentRate != 0.0f)
		{
			FTimespan PlayPosNow = MediaPlayer->GetTime();
			if (PlayPosNow >= FTimespan::Zero())
			{
				// If the playback position wrapped around, ie the player has looped we
				// mark subtitles as dirty to ensure the current subtitles get cleared.
				if ((CurrentRate > 0.0f && PlayPosNow < Internal->PrevPlayerTime) ||
					(CurrentRate < 0.0f && PlayPosNow > Internal->PrevPlayerTime))
				{
					Internal->bIsDirty = true;
				}
				Internal->PrevPlayerTime = PlayPosNow;

				// Check if we are past the end of the most recent subtitle in case
				// the subtitle manager does not send an empty subtitle.
				if (CurrentRate > 0.0f && PlayPosNow > Internal->ClearAfterPlayerTime)
				{
					Internal->ClearAfterPlayerTime = FTimespan::MaxValue();
					Internal->bIsDirty = true;
				}
			}
			else
			{
				Internal->bIsDirty = true;
			}
		}
	}

	// Get the new subtitle samples. They are delivered just in time and there is no
	// need for us to check if they are due. We do need however to track when to
	// remove them in case the subtitle decoder does not send an empty sample.
	TArray<TSharedPtr<IMediaOverlaySample, ESPMode::ThreadSafe>> OverlaySamples;
	TSharedPtr<IMediaOverlaySample, ESPMode::ThreadSafe> NewSample;
	FTimespan ClearAfterPlayerTime = FTimespan::MinValue();
	while(Internal->SubtitleQueue->Dequeue(NewSample))
	{
		if (NewSample.IsValid())
		{
			ClearAfterPlayerTime = FMath::Max(ClearAfterPlayerTime, NewSample->GetTime().GetTime() + NewSample->GetDuration());
			OverlaySamples.Emplace(MoveTemp(NewSample));
			Internal->bIsDirty = true;
		}
	}
	if (ClearAfterPlayerTime > FTimespan::MinValue())
	{
		Internal->ClearAfterPlayerTime = ClearAfterPlayerTime;
	}


	if (Internal->bIsDirty)
	{
		Internal->bIsDirty = false;

		Canvas->ClearChildren();

		// Create widgets for overlays
		for(const auto& Sample : OverlaySamples)
		{
			auto RichTextBlock = SNew(SRichTextBlock)
				.AutoWrapText(true)
				.Justification(ETextJustify::Center)
				.Text(Sample->GetText());

			RichTextBlock->SetTextStyle(Configuration->TextStyle);
			//RichTextBlock->SetBorderImage(&GeneratedBackgroundBorder);

			const TOptional<FVector2D> Position = Sample->GetPosition();
			if (Position.IsSet())
			{
				Canvas->AddSlot()
					.Alignment(FVector2D(0.0f, 0.0f))
					.Anchors(FAnchors(0.0f, 0.0f, 0.0f, 0.0f))
					.AutoSize(true)
					.Offset(FMargin(Position->X, Position->Y, 0.0f, 0.0f))
					[
						RichTextBlock
					];
			}
			else
			{
				Canvas->AddSlot()
					.Alignment(FVector2D(0.0f, 1.0f))
					.Anchors(FAnchors(0.1f, 0.8f, 0.9f, 0.9f))
					.AutoSize(true)
					[
						RichTextBlock
					];
			}
		}
		SlatePrepass(AllottedGeometry.GetAccumulatedLayoutTransform().GetScale());
	}
}
