// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/SharedPointer.h"
#include "Templates/PimplPtr.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"

class SConstraintCanvas;
class UMediaPlayer;


/**
 * Draws text overlays for the UMediaPlayer asset editor.
 */
class SMediaPlayerEditorOverlay
	: public SCompoundWidget
{
public:

	SLATE_BEGIN_ARGS(SMediaPlayerEditorOverlay) { }
	SLATE_END_ARGS()

public:

	/**
	 * Construct this widget
	 *
	 * @param InArgs The declaration data for this widget.
	 * @param InMediaPlayer The UMediaPlayer asset to show the details for.
	 */
	void Construct(const FArguments& InArgs, UMediaPlayer& InMediaPlayer);

public:

	//~ SWidget interface

	void Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime) override;

private:

	/** Generated configuration. */
	struct FConfig;
	TPimplPtr<FConfig> Configuration;

	/** The canvas to draw into. */
	TSharedPtr<SConstraintCanvas> Canvas;

	/** The media player whose video texture is shown in this widget. */
	UMediaPlayer* MediaPlayer;

	/** Internals */
	class FInternal;
	TSharedPtr<FInternal, ESPMode::ThreadSafe> Internal;
};
