// Copyright Epic Games, Inc. All Rights Reserved.

#include "SMediaPlayerEditorPlaylist.h"

#include "Containers/Array.h"
#include "Editor.h"
#include "Styling/AppStyle.h"
#include "FileHelpers.h"
#include "IMediaEventSink.h"
#include "IMediaPlayer.h"
#include "MediaPlayer.h"
#include "MediaPlaylist.h"
#include "UObject/Package.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Views/SListView.h"
#include "Widgets/SMediaSourceTableRow.h"


#define LOCTEXT_NAMESPACE "SMediaPlayerEditorMedia"


/* SMediaPlayerEditorPlaylist structors
 *****************************************************************************/

SMediaPlayerEditorPlaylist::SMediaPlayerEditorPlaylist()
	: MediaPlayer(nullptr)
{ }


SMediaPlayerEditorPlaylist::~SMediaPlayerEditorPlaylist()
{
	if (MediaPlayer != nullptr)
	{
		MediaPlayer->OnMediaEvent().RemoveAll(this);
	}

	FCoreUObjectDelegates::OnObjectPropertyChanged.RemoveAll(this);
}


/* SMediaPlayerEditorPlaylist interface
 *****************************************************************************/

void SMediaPlayerEditorPlaylist::Construct(const FArguments& InArgs, UMediaPlayer& InMediaPlayer, const TSharedRef<ISlateStyle>& InStyle)
{
	MediaPlayer = &InMediaPlayer;
	Style = InStyle;

	ChildSlot
	[
		SNew(SVerticalBox)

		+ SVerticalBox::Slot()
			.AutoHeight()
			.Padding(2.0f)
			[
				SNew(SHorizontalBox)

				+ SHorizontalBox::Slot()
					.AutoWidth()
					.VAlign(VAlign_Center)
					[
						// play list name
						SNew(STextBlock)
							.Text_Lambda([this]() -> FText
							{
								UMediaPlaylist& Playlist = MediaPlayer->GetPlaylistRef();
								if ((Playlist.GetFlags() & RF_Transient) != 0)
								{
									return LOCTEXT("UnsavedPlaylistLabel", "[Unsaved play list]");
								}
								else
								{
									FString PlaylistName;
									MediaPlayer->GetPlaylistRef().GetName(PlaylistName);
									return FText::FromString(PlaylistName);
								}
							})
					]

				+ SHorizontalBox::Slot()
					.AutoWidth()
					.Padding(4.0f, 0.0f, 0.0f, 0.0f)
					.VAlign(VAlign_Center)
					[
						// browse button
						SNew(SButton)
							.ButtonStyle(FAppStyle::Get(), "HoverHintOnly")
							.ContentPadding(4.0f)
							.ForegroundColor(FSlateColor::UseForeground())
							.IsEnabled_Lambda([this]() -> bool {
								return ((MediaPlayer->GetPlaylistRef().GetFlags() & RF_Transient) == 0);
							})
							.OnClicked_Lambda([this]() -> FReply
							{
								TArray<UObject*> AssetsToSync;
								AssetsToSync.Add(MediaPlayer->GetPlaylist());
								GEditor->SyncBrowserToObjects(AssetsToSync);
								return FReply::Handled();
							})
							.ToolTipText(LOCTEXT("FindPlaylistButtonToolTip", "Find this playlist in the Content Browser"))
							[
								SNew(SImage)
									.ColorAndOpacity(FSlateColor::UseForeground())
									.Image(FAppStyle::GetBrush("Icons.Search"))
							]
					]

				+ SHorizontalBox::Slot()
					.FillWidth(1.0f)
					.HAlign(HAlign_Left)
					.VAlign(VAlign_Center)
					[
						// save button
						SNew(SButton)
							.ButtonStyle(FAppStyle::Get(), "HoverHintOnly")
							.ContentPadding(4.0f)
							.ForegroundColor(FSlateColor::UseForeground())
							.IsEnabled_Lambda([this]() -> bool {
								UMediaPlaylist& Playlist = MediaPlayer->GetPlaylistRef();
								UPackage* Package = Playlist.GetOutermost();
								return ((Playlist.GetFlags() & RF_Transient) != 0) || ((Package != nullptr) && Package->IsDirty());
							})
							.OnClicked(this, &SMediaPlayerEditorPlaylist::HandleSavePlaylistButtonClicked)
							.ToolTipText(LOCTEXT("SavePlaylistButtonToolTip", "Save this playlist"))
							[
								SNew(SImage)
									.ColorAndOpacity(FSlateColor::UseForeground())
									.Image_Lambda([this]() -> const FSlateBrush*
									{
										return FAppStyle::GetBrush(
											((MediaPlayer->GetPlaylistRef().GetFlags() & RF_Transient) != 0)
												? "LevelEditor.Save"
												: "LevelEditor.SaveAs"
										);
									})
							]
					]

				+ SHorizontalBox::Slot()
					.AutoWidth()
					.Padding(2.0f, 0.0f, 0.0f, 0.0f)
					.VAlign(VAlign_Center)
					[
						// previous item button
						SNew(SButton)
							.IsEnabled_Lambda([this]() -> bool {
								return (MediaPlayer->GetPlaylistIndex() > 0);
							})
							.OnClicked_Lambda([this]() -> FReply
							{
								MediaPlayer->Previous();
								return FReply::Handled();
							})
							.ToolTipText(LOCTEXT("PreviousPlaylistItemButtonToolTip", "Jump to the previous item in the playlist"))
							[
								SNew(SImage)
									.Image(FAppStyle::GetBrush("Icons.CircleArrowLeft"))
							]
					]

				+ SHorizontalBox::Slot()
					.AutoWidth()
					.Padding(2.0f, 0.0f, 0.0f, 0.0f)
					.VAlign(VAlign_Center)
					[
						// next item button
						SNew(SButton)
							.IsEnabled_Lambda([this]() -> bool {
								return (MediaPlayer->GetPlaylistIndex() + 1 < MediaPlayer->GetPlaylistRef().Num());
							})
							.OnClicked_Lambda([this]() -> FReply
							{
								MediaPlayer->Next();
								return FReply::Handled();
							})
							.ToolTipText(LOCTEXT("NextPlaylistItemButtonToolTip", "Jump to the next item in the playlist"))
							[
								SNew(SImage)
								.Image(FAppStyle::GetBrush("Icons.CircleArrowRight"))
							]
					]
			]

		+ SVerticalBox::Slot()
			.FillHeight(1.0f)
			[
				SNew(SBorder)
					.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
					.Padding(0.0f)
					[
						// media source list
						SAssignNew(MediaSourceListView, SListView<TSharedPtr<FMediaSourceTableEntry>>)
							.ListItemsSource(&MediaSourceList)
							.SelectionMode(ESelectionMode::Single)
							.OnGenerateRow_Lambda([this](TSharedPtr<FMediaSourceTableEntry> Entry, const TSharedRef<STableViewBase>& OwnerTable) -> TSharedRef<ITableRow>
							{
								return SNew(SMediaSourceTableRow, OwnerTable)
									.Entry(Entry)
									.Opened_Lambda([this, Entry]() { return MediaPlayer->GetPlaylistIndex() == Entry->Index; })
									.Style(Style);
							})
							.OnMouseButtonDoubleClick_Lambda([this](TSharedPtr<FMediaSourceTableEntry> InItem)
							{
								if (InItem.IsValid())
								{
									MediaPlayer->OpenPlaylistIndex(&MediaPlayer->GetPlaylistRef(), InItem->Index);
								}
							})
							.HeaderRow
							(
								SNew(SHeaderRow)

								+ SHeaderRow::Column("Icon")
									.DefaultLabel(LOCTEXT("PlaylistIconColumnHeader", " "))
									.FixedWidth(12.0f)

								+ SHeaderRow::Column("Index")
									.DefaultLabel(LOCTEXT("PlaylistIndexColumnHeader", "#"))
									.FillWidth(0.1f)

								+ SHeaderRow::Column("Source")
									.DefaultLabel(LOCTEXT("PlaylistSourceColumnHeader", "Media Source"))
									.FillWidth(0.5f)

								+ SHeaderRow::Column("Type")
									.DefaultLabel(LOCTEXT("PlaylistTypeColumnHeader", "Type"))
									.FillWidth(0.4f)
							)
					]
			]
	];

	FCoreUObjectDelegates::OnObjectPropertyChanged.AddSP(this, &SMediaPlayerEditorPlaylist::HandleCoreObjectPropertyChanged);
	MediaPlayer->OnMediaEvent().AddSP(this, &SMediaPlayerEditorPlaylist::HandleMediaPlayerMediaEvent);

	ReloadMediaSourceList();
}


/* SMediaPlayerEditorPlaylist implementation
 *****************************************************************************/

void SMediaPlayerEditorPlaylist::ReloadMediaSourceList()
{
	MediaSourceList.Reset();

	UMediaPlaylist& Playlist = MediaPlayer->GetPlaylistRef();

	for (int32 EntryIndex = 0; EntryIndex < Playlist.Num(); ++EntryIndex)
	{
		MediaSourceList.Add(MakeShareable(new FMediaSourceTableEntry(EntryIndex, Playlist.Get(EntryIndex))));
	}

	MediaSourceListView->RequestListRefresh();
}


/* SMediaPlayerEditorPlaylist callbacks
 *****************************************************************************/

void SMediaPlayerEditorPlaylist::HandleCoreObjectPropertyChanged(UObject* Object, struct FPropertyChangedEvent& ChangedEvent)
{
	if ((Object != nullptr) && (Object == &MediaPlayer->GetPlaylistRef()))
	{
		ReloadMediaSourceList();
	}
}


void SMediaPlayerEditorPlaylist::HandleMediaPlayerMediaEvent(EMediaEvent Event)
{
	if ((Event == EMediaEvent::MediaClosed) || (Event == EMediaEvent::MediaOpened))
	{
		ReloadMediaSourceList();
	}
}


FReply SMediaPlayerEditorPlaylist::HandleSavePlaylistButtonClicked()
{
	UMediaPlaylist& Playlist = MediaPlayer->GetPlaylistRef();

	// first save any transient media sources
	for (int32 MediaSourceIndex = 0; MediaSourceIndex < Playlist.Num(); ++MediaSourceIndex)
	{
		UMediaSource* MediaSource = Playlist.Get(MediaSourceIndex);

		if ((MediaSource != nullptr) && MediaSource->HasAnyFlags(RF_Transient))
		{
			TArray<UObject*> MediaSourcesToSave;
			TArray<UObject*> SavedMediaSources;

			MediaSourcesToSave.Add(MediaSource);
			FEditorFileUtils::SaveAssetsAs(MediaSourcesToSave, SavedMediaSources);

			if ((SavedMediaSources.Num() != 1) || (SavedMediaSources[0] == MediaSourcesToSave[0]))
			{
				return FReply::Handled(); // user canceled
			}

			Playlist.Replace(MediaSourceIndex, CastChecked<UMediaSource>(SavedMediaSources[0]));
		}
	}

	// then save play list itself
	if ((Playlist.GetFlags() & RF_Transient) == 0)
	{
		// save existing play list asset
		TArray<UPackage*> PackagesToSave;

		PackagesToSave.Add(Playlist.GetOutermost());
		FEditorFileUtils::PromptForCheckoutAndSave(PackagesToSave, false /*bCheckDirty*/, false /*bPromptToSave*/);
	}
	else
	{
		// create & save new play list asset
		TArray<UObject*> PlaylistsToSave;
		TArray<UObject*> SavedPlaylists;

		PlaylistsToSave.Add(&Playlist);
		FEditorFileUtils::SaveAssetsAs(PlaylistsToSave, SavedPlaylists);

		if ((SavedPlaylists.Num() != 1) || (SavedPlaylists[0] == &Playlist))
		{
			return FReply::Handled(); // user canceled
		}

		MediaPlayer->OpenPlaylistIndex(CastChecked<UMediaPlaylist>(SavedPlaylists[0]), MediaPlayer->GetPlaylistIndex());
	}

	return FReply::Handled();
}


#undef LOCTEXT_NAMESPACE
