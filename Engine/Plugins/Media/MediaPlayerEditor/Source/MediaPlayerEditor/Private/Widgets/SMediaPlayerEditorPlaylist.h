// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Styling/ISlateStyle.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/Views/STableViewBase.h"
#include "Widgets/Views/STableRow.h"

class UMediaPlayer;
struct FMediaSourceTableEntry;
enum class EMediaEvent;


/**
 * Implements the play list of the MediaPlayer asset editor.
 */
class SMediaPlayerEditorPlaylist
	: public SCompoundWidget
{
public:

	SLATE_BEGIN_ARGS(SMediaPlayerEditorPlaylist) { }
	SLATE_END_ARGS()

public:

	/** Default constructor. */
	SMediaPlayerEditorPlaylist();

	/** Destructor. */
	~SMediaPlayerEditorPlaylist();

public:

	/**
	 * Construct this widget
	 *
	 * @param InArgs The declaration data for this widget.
	 * @param InMediaPlayer The MediaPlayer asset to show the details for.
	 * @param InStyleSet The style set to use.
	 */
	void Construct(const FArguments& InArgs, UMediaPlayer& InMediaPlayer, const TSharedRef<ISlateStyle>& InStyle);

protected:

	/** Reload the list of media sources in the play list. */
	void ReloadMediaSourceList();

private:

	/** Callback for when a UObject property has changed (used to monitor play list changed). */
	void HandleCoreObjectPropertyChanged(UObject* Object, struct FPropertyChangedEvent& ChangedEvent);

	/** Callback for media player events. */
	void HandleMediaPlayerMediaEvent(EMediaEvent Event);

	/** Callback for clicking the Save button. */
	FReply HandleSavePlaylistButtonClicked();

private:

	/** Pointer to the MediaPlayer asset that is being viewed. */
	UMediaPlayer* MediaPlayer;

	/** The list of media sources in the current play list. */
	TArray<TSharedPtr<FMediaSourceTableEntry>> MediaSourceList;

	/** Media source list view. */
	TSharedPtr<SListView<TSharedPtr<FMediaSourceTableEntry>>> MediaSourceListView;

	/** The widget style to use. */
	TSharedPtr<ISlateStyle> Style;
};
