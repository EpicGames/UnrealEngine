// Copyright Epic Games, Inc. All Rights Reserved.

#include "SMediaPlayerEditorStats.h"

#include "Styling/AppStyle.h"
#include "MediaPlayer.h"
#include "MediaPlayerFacade.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Layout/SScrollBox.h"
#include "Widgets/Text/STextBlock.h"
#include "HAL/PlatformApplicationMisc.h"

#define LOCTEXT_NAMESPACE "SMediaPlayerEditorStats"


/* SMediaPlayerEditorStats interface
 *****************************************************************************/

void SMediaPlayerEditorStats::Construct(const FArguments& InArgs, UMediaPlayer& InMediaPlayer, const TSharedRef<ISlateStyle>& InStyle)
{
	MediaPlayer = &InMediaPlayer;

	ChildSlot
	[
		SNew(SVerticalBox)

		+ SVerticalBox::Slot()
			.FillHeight(1.0f)
			[
				SNew(SScrollBox)

				+ SScrollBox::Slot()
					.Padding(4.0f)
					[
						SAssignNew(StatsTextBlock, STextBlock)
							.Text(this, &SMediaPlayerEditorStats::HandleStatsTextBlockText)
					]
			]

		+ SVerticalBox::Slot()
			.AutoHeight()
			[
				SNew(SBorder)
					.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
					.HAlign(HAlign_Right)
					.Padding(2.0f)
					[
						SNew(SButton)
							.Text(LOCTEXT("CopyClipboardButtonText", "Copy to Clipboard"))
							.ToolTipText(LOCTEXT("CopyClipboardButtonHint", "Copy the media statistics to the the clipboard"))
							.OnClicked_Lambda([this]() -> FReply {
								FPlatformApplicationMisc::ClipboardCopy(*StatsTextBlock->GetText().ToString());
								return FReply::Handled();
							})
					]
			]
	];
}


/* SMediaPlayerEditorStats callbacks
 *****************************************************************************/

FText SMediaPlayerEditorStats::HandleStatsTextBlockText() const
{
	if (MediaPlayer->GetUrl().IsEmpty())
	{
		return LOCTEXT("NoMediaOpened", "No media opened");
	}

	return FText::FromString(MediaPlayer->GetPlayerFacade()->GetStats());
}


#undef LOCTEXT_NAMESPACE
