// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Styling/ISlateStyle.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"

class STextBlock;
class UMediaPlayer;

/**
 * Implements the stats panel of the MediaPlayer asset editor.
 */
class SMediaPlayerEditorStats
	: public SCompoundWidget
{
public:

	SLATE_BEGIN_ARGS(SMediaPlayerEditorStats) { }
	SLATE_END_ARGS()

public:

	/**
	 * Construct this widget
	 *
	 * @param InArgs The declaration data for this widget.
	 * @param InMediaPlayer The MediaPlayer asset to show the information for.
	 * @param InStyleSet The style set to use.
	 */
	void Construct(const FArguments& InArgs, UMediaPlayer& InMediaPlayer, const TSharedRef<ISlateStyle>& InStyle);

private:

	/** Callback for getting the text of the statistics text block. */
	FText HandleStatsTextBlockText() const;

private:

	/** Pointer to the MediaPlayer asset that is being viewed. */
	UMediaPlayer* MediaPlayer;

	/** Text block for media statistics. */
	TSharedPtr<STextBlock> StatsTextBlock;
};
