// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/SMediaPlayerEditorViewer.h"

#include "Internationalization/Culture.h"
#include "Modules/ModuleManager.h"

#include "IMediaModule.h"
#include "IMediaEventSink.h"
#include "IMediaPlayerFactory.h"
#include "MediaPlayer.h"
#include "MediaPlaylist.h"

#include "Styling/AppStyle.h"
#include "Framework/Application/SlateApplication.h"
#include "Framework/Commands/UIAction.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Framework/Notifications/NotificationManager.h"
#include "SMediaPlayerSlider.h"
#include "Styling/CoreStyle.h"
#include "Textures/SlateIcon.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SOverlay.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Images/SThrobber.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SComboButton.h"
#include "Widgets/Input/SEditableTextBox.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Layout/SSpacer.h"
#include "Widgets/Notifications/SNotificationList.h"
#include "Widgets/Notifications/SProgressBar.h"
#include "Widgets/Text/STextBlock.h"

#include "Shared/MediaPlayerEditorSettings.h"
#include "SMediaPlayerEditorCache.h"
#include "SMediaPlayerEditorViewport.h"

#include "EditorWidgetsModule.h"
#include "Models/MediaPlayerEditorCommands.h"
#include "Modules/ModuleManager.h"
#include "Widgets/SMediaControls.h"


#define LOCTEXT_NAMESPACE "SMediaPlayerEditorViewer"

namespace UE::MediaPlayerEditor
{
	FTimespan GetVideoFrameDuration(const UMediaPlayer* InMediaPlayer)
	{
		// Get selected track's frame rate.
		constexpr int32 SelectedTrackIndex = INDEX_NONE;
		constexpr int32 SelectedFormatIndex = INDEX_NONE;
		const float Framerate = InMediaPlayer ? InMediaPlayer->GetVideoTrackFrameRate(SelectedTrackIndex, SelectedFormatIndex) : 0.0f;
		return Framerate > 0.0f ? FTimespan::FromSeconds(1.0f / Framerate) : FTimespan::Zero();
	}
}

/* SMediaPlayerEditorPlayer structors
 *****************************************************************************/

SMediaPlayerEditorViewer::SMediaPlayerEditorViewer()
	: DragOver(false)
	, DragValid(false)
	, MediaPlayer(nullptr)
{ }


SMediaPlayerEditorViewer::~SMediaPlayerEditorViewer()
{
	if (MediaPlayer != nullptr)
	{
		MediaPlayer->OnMediaEvent().RemoveAll(this);
	}
}


/* SMediaPlayerEditorPlayer interface
 *****************************************************************************/

void SMediaPlayerEditorViewer::Construct(const FArguments& InArgs, UMediaPlayer& InMediaPlayer,
	UMediaTexture* InMediaTexture, const TSharedRef<ISlateStyle>& InStyle,
	bool bInIsSoundEnabled)
{
	MediaPlayer = &InMediaPlayer;
	Style = InStyle;
	CommandListWeak = InArgs._Commands;

	// initialize media player asset
	MediaPlayer->OnMediaEvent().AddSP(this, &SMediaPlayerEditorViewer::HandleMediaPlayerMediaEvent);
	MediaPlayer->SetDesiredPlayerName(NAME_None);

	TWeakObjectPtr<UMediaPlayer> MediaPlayerWeak(&InMediaPlayer);
	
	FName DesiredPlayerName = GetDefault<UMediaPlayerEditorSettings>()->DesiredPlayerName;

	if (DesiredPlayerName != NAME_None)
	{
		IMediaModule* MediaModule = FModuleManager::LoadModulePtr<IMediaModule>("Media");

		if ((MediaModule != nullptr) && (MediaModule->GetPlayerFactory(DesiredPlayerName) != nullptr))
		{
			MediaPlayer->SetDesiredPlayerName(DesiredPlayerName);
		}
	}

	// initialize capture source menu
	FMenuBuilder SourceMenuBuilder(true, nullptr);
	{
		SourceMenuBuilder.BeginSection("CaptureDevicesSection", LOCTEXT("CaptureDevicesSection", "Capture Devices"));
		{
			SourceMenuBuilder.AddSubMenu(
				LOCTEXT("AudioMenuLabel", "Audio"),
				LOCTEXT("AudioMenuTooltip", "Available audio capture devices"),
				FNewMenuDelegate::CreateRaw(this, &SMediaPlayerEditorViewer::HandleAudioCaptureDevicesMenuNewMenu),
				false,
				FSlateIcon()
			);

			SourceMenuBuilder.AddSubMenu(
				LOCTEXT("VideoMenuLabel", "Video"),
				LOCTEXT("VideoMenuTooltip", "Available video capture devices"),
				FNewMenuDelegate::CreateRaw(this, &SMediaPlayerEditorViewer::HandleVideoCaptureDevicesMenuNewMenu),
				false,
				FSlateIcon()
			);
		}
		SourceMenuBuilder.EndSection();
	}

	// initialize settings menu
	FMenuBuilder SettingsMenuBuilder(true, nullptr);
	{
		SettingsMenuBuilder.BeginSection("PlayerSection", LOCTEXT("PlayerSection", "Player"));
		{
			SettingsMenuBuilder.AddSubMenu(
				LOCTEXT("DecoderMenuLabel", "Decoder"),
				LOCTEXT("DecoderMenuTooltip", "Select the desired media decoder"),
				FNewMenuDelegate::CreateRaw(this, &SMediaPlayerEditorViewer::HandleDecoderMenuNewMenu),
				false,
				FSlateIcon()
			);
		}
		SettingsMenuBuilder.EndSection();

		SettingsMenuBuilder.BeginSection("TracksSection", LOCTEXT("TracksSection", "Tracks"));
		{
			SettingsMenuBuilder.AddSubMenu(
				LOCTEXT("AudioTrackMenuLabel", "Audio"),
				LOCTEXT("AudioTrackMenuTooltip", "Select the active audio track"),
				FNewMenuDelegate::CreateRaw(this, &SMediaPlayerEditorViewer::HandleTrackMenuNewMenu, EMediaPlayerTrack::Audio),
				false,
				FSlateIcon()
			);

			SettingsMenuBuilder.AddSubMenu(
				LOCTEXT("CaptionTrackMenuLabel", "Captions"),
				LOCTEXT("CaptionTrackMenuTooltip", "Select the active closed caption track"),
				FNewMenuDelegate::CreateRaw(this, &SMediaPlayerEditorViewer::HandleTrackMenuNewMenu, EMediaPlayerTrack::Caption),
				false,
				FSlateIcon()
			);

			SettingsMenuBuilder.AddSubMenu(
				LOCTEXT("SubtitleTrackMenuLabel", "Subtitles"),
				LOCTEXT("SubtitleTrackMenuTooltip", "Select the active subtitle track"),
				FNewMenuDelegate::CreateRaw(this, &SMediaPlayerEditorViewer::HandleTrackMenuNewMenu, EMediaPlayerTrack::Subtitle),
				false,
				FSlateIcon()
			);

			SettingsMenuBuilder.AddSubMenu(
				LOCTEXT("TextTrackMenuLabel", "Text"),
				LOCTEXT("TextTrackMenuTooltip", "Select the active generic text track"),
				FNewMenuDelegate::CreateRaw(this, &SMediaPlayerEditorViewer::HandleTrackMenuNewMenu, EMediaPlayerTrack::Text),
				false,
				FSlateIcon()
			);

			SettingsMenuBuilder.AddSubMenu(
				LOCTEXT("VideoTrackMenuLabel", "Video"),
				LOCTEXT("VideoTrackMenuTooltip", "Select the active video track"),
				FNewMenuDelegate::CreateRaw(this, &SMediaPlayerEditorViewer::HandleTrackMenuNewMenu, EMediaPlayerTrack::Video),
				false,
				FSlateIcon()
			);
		}
		SettingsMenuBuilder.EndSection();

		SettingsMenuBuilder.BeginSection("FormatsSection", LOCTEXT("FormatsSection", "Formats"));
		{
			SettingsMenuBuilder.AddSubMenu(
				LOCTEXT("AudioFormatMenuLabel", "Audio"),
				LOCTEXT("AudioFormatMenuTooltip", "Select the active audio format"),
				FNewMenuDelegate::CreateRaw(this, &SMediaPlayerEditorViewer::HandleFormatMenuNewMenu, EMediaPlayerTrack::Audio),
				false,
				FSlateIcon()
			);

			SettingsMenuBuilder.AddSubMenu(
				LOCTEXT("CaptionFormatMenuLabel", "Captions"),
				LOCTEXT("CaptionFormatMenuTooltip", "Select the active closed caption format"),
				FNewMenuDelegate::CreateRaw(this, &SMediaPlayerEditorViewer::HandleFormatMenuNewMenu, EMediaPlayerTrack::Caption),
				false,
				FSlateIcon()
			);

			SettingsMenuBuilder.AddSubMenu(
				LOCTEXT("SubtitleFormatMenuLabel", "Subtitles"),
				LOCTEXT("SubtitleFormatMenuTooltip", "Select the active subtitle format"),
				FNewMenuDelegate::CreateRaw(this, &SMediaPlayerEditorViewer::HandleFormatMenuNewMenu, EMediaPlayerTrack::Subtitle),
				false,
				FSlateIcon()
			);

			SettingsMenuBuilder.AddSubMenu(
				LOCTEXT("TextFormatMenuLabel", "Text"),
				LOCTEXT("TextFormatMenuTooltip", "Select the active generic text format"),
				FNewMenuDelegate::CreateRaw(this, &SMediaPlayerEditorViewer::HandleFormatMenuNewMenu, EMediaPlayerTrack::Text),
				false,
				FSlateIcon()
			);

			SettingsMenuBuilder.AddSubMenu(
				LOCTEXT("VideoFormatMenuLabel", "Video"),
				LOCTEXT("VideoFormatMenuTooltip", "Select the active video format"),
				FNewMenuDelegate::CreateRaw(this, &SMediaPlayerEditorViewer::HandleFormatMenuNewMenu, EMediaPlayerTrack::Video),
				false,
				FSlateIcon()
			);
		}
		SettingsMenuBuilder.EndSection();

		SettingsMenuBuilder.BeginSection("ViewSection", LOCTEXT("ViewSection", "View"));
		{
			SettingsMenuBuilder.AddSubMenu(
				LOCTEXT("ScaleMenuLabel", "Scale"),
				LOCTEXT("ScaleMenuTooltip", "Select the video viewport's scaling mode"),
				FNewMenuDelegate::CreateRaw(this, &SMediaPlayerEditorViewer::HandleScaleMenuNewMenu),
				false,
				FSlateIcon()
			);

			SettingsMenuBuilder.AddSubMenu(
				LOCTEXT("BackgroundMenuLabel", "Background"),
				LOCTEXT("BackgroundMenuTooltip", "Change the background behind the media"),
				FNewMenuDelegate::CreateRaw(this, &SMediaPlayerEditorViewer::HandleBackgroundMenuNewMenu),
				false,
				FSlateIcon()
			);

			SettingsMenuBuilder.AddSubMenu(
				LOCTEXT("TimerUnitMenuLabel", "Timer Units"),
				LOCTEXT("TimerUnitMenuTooltip", "Change the timer units"),
				FNewMenuDelegate::CreateRaw(this, &SMediaPlayerEditorViewer::HandleTimerUnitMenuNewMenu),
				false,
				FSlateIcon()
			);

			SettingsMenuBuilder.AddMenuEntry(
				LOCTEXT("ShowTextOverlaysMenuLabel", "Show Text Overlays"),
				LOCTEXT("ShowTextOverlaysMenuTooltip", "Show caption and subtitle text overlays"),
				FSlateIcon(),
				FUIAction(
					FExecuteAction::CreateLambda([]{
						UMediaPlayerEditorSettings* Settings = GetMutableDefault<UMediaPlayerEditorSettings>();
						Settings->ShowTextOverlays = !Settings->ShowTextOverlays;
						Settings->SaveConfig();
					}),
					FCanExecuteAction(),
					FIsActionChecked::CreateLambda([]{ return GetDefault<UMediaPlayerEditorSettings>()->ShowTextOverlays; })
				),
				NAME_None,
				EUserInterfaceActionType::ToggleButton
			);
		}
		SettingsMenuBuilder.EndSection();
	}

	// widget contents
	ChildSlot
	[
		SNew(SOverlay)

		+ SOverlay::Slot()
			[
				SNew(SVerticalBox)

				+ SVerticalBox::Slot()
					.AutoHeight()
					[
						// url area
						SNew(SHorizontalBox)
							.Visibility(InArgs._bShowUrl ? EVisibility::Visible : EVisibility::Collapsed)

						+ SHorizontalBox::Slot()
							.AutoWidth()
							[
								// capture source drop-down
								SNew(SComboButton)
									.ContentPadding(0.0f)
									.ButtonContent()
									[
										SNew(SImage)
											.Image(InStyle->GetBrush("MediaPlayerEditor.SourceButton"))
									]
									.ButtonStyle(FAppStyle::Get(), "ToggleButton")
									.ForegroundColor(FSlateColor::UseForeground())
									.MenuContent()
									[
										SourceMenuBuilder.MakeWidget()
									]
							]

						+ SHorizontalBox::Slot()
							.FillWidth(1.0f)
							.Padding(4.0f, 0.0f, 0.0f, 0.0f)
							[
								// url box
								SAssignNew(UrlTextBox, SEditableTextBox)
									.BackgroundColor_Lambda([this]() -> FSlateColor {
										return (MediaPlayer->IsPreparing() || LastUrl.IsEmpty()) ? FLinearColor::White : FLinearColor::Red;
									})
									.ClearKeyboardFocusOnCommit(true)
									.HintText(LOCTEXT("UrlTextBoxHint", "Media URL"))
									.Text_Lambda([this]() -> FText {
										return LastUrl.IsEmpty() ? FText::FromString(MediaPlayer->GetUrl()) : LastUrl;
									})
									.ToolTipText(LOCTEXT("UrlTextBoxToolTip", "Enter the URL of a media source"))
									.OnKeyDownHandler(this, &SMediaPlayerEditorViewer::HandleUrlBoxKeyDown)
									.OnTextCommitted_Lambda([this](const FText& InText, ETextCommit::Type InCommitType){
										if (InCommitType == ETextCommit::OnEnter)
										{
											OpenUrl(InText);
										}
									})
							]

						+ SHorizontalBox::Slot()
							.AutoWidth()
							.Padding(4.0f, 0.0f, 0.0f, 0.0f)
							[
								// go button
								SNew(SButton)
									.ToolTipText_Lambda([this]() {
										return ((UrlTextBox->GetText().ToString() == MediaPlayer->GetUrl()) && !MediaPlayer->GetUrl().IsEmpty())
											? LOCTEXT("ReloadButtonToolTip", "Reload the current media URL")
											: LOCTEXT("GoButtonToolTip", "Open the specified media URL");
									})
									.IsEnabled_Lambda([this]{
										return !UrlTextBox->GetText().IsEmpty();
									})
									.OnClicked_Lambda([this]{
										OpenUrl(UrlTextBox->GetText());
										return FReply::Handled();
									})
									[
										SNew(SImage)
											.Image_Lambda([this]() {
												return ((UrlTextBox->GetText().ToString() == MediaPlayer->GetUrl()) && !MediaPlayer->GetUrl().IsEmpty())
													? Style->GetBrush("MediaPlayerEditor.ReloadButton")
													: Style->GetBrush("MediaPlayerEditor.GoButton");
											})
									]
							]
					]

				+ SVerticalBox::Slot()
					.FillHeight(1.0f)
					.Padding(0.0f, 4.0f, 0.0f, 0.0f)
					[
						SNew(SBorder)
							.BorderImage(FCoreStyle::Get().GetBrush("BlackBrush"))
							.Padding(0.0f)
							[
								// movie area
								SAssignNew(PlayerViewport, SMediaPlayerEditorViewport, InMediaPlayer, InMediaTexture, InStyle, bInIsSoundEnabled, InArgs._Commands)
							]
					]

				+ SVerticalBox::Slot()
					.AutoHeight()
					.Padding(0.0f, 2.0f, 0.0f, 0.0f)
					.VAlign(VAlign_Center)
					[
						// playback controls
						SNew(SBorder)
							.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
							.ForegroundColor(FLinearColor::Gray)
							.Padding(6.0f)
							[
								SNew(SVerticalBox)

								+ SVerticalBox::Slot()
									.AutoHeight()
									[
										SNew(SOverlay)

										+ SOverlay::Slot()
											.Padding(2.0f, 0.0f, 2.0f, 0.0f)
											.VAlign(VAlign_Top)
											[
												// cache visualization
												SNew(SMediaPlayerEditorCache, InMediaPlayer, InStyle)
											]

										+ SOverlay::Slot()
											.VAlign(VAlign_Top)
											[
												// time slider
												SNew(SMediaPlayerSlider, MakeArrayView(&MediaPlayerWeak, 1))
												.ToolTipText( LOCTEXT("PlaybackPosition", "Current Playback Position"))
												.Style(&InStyle->GetWidgetStyle<FSliderStyle>("MediaPlayerEditor.Scrubber"))
											]

										+ SOverlay::Slot()
											.VAlign(VAlign_Center)
											[
												// animated progress bar
												SNew(SProgressBar)
													.ToolTipText(LOCTEXT("PreparingTooltip", "Preparing..."))
													.Visibility_Lambda([this]() -> EVisibility
													{
														return MediaPlayer->IsPreparing() ? EVisibility::Visible : EVisibility::Hidden;
													})
											]
									]

								+ SVerticalBox::Slot()
									.AutoHeight()
									.Padding(0.0f, 2.0f, 0.0f, 0.0f)
									[
										SNew(SHorizontalBox)

										+ SHorizontalBox::Slot()
											.AutoWidth()
											.VAlign(VAlign_Center)
											.HAlign(HAlign_Left)
											[
												SNew(SHorizontalBox)

												// timer
												+ SHorizontalBox::Slot()
													.MinWidth(85.f)
													.Padding(4.0f, 0.0f, 0.0f, 0.0f)
													.VAlign(VAlign_Center)
													.HAlign(HAlign_Right)
													[
														SNew(STextBlock)
															.Text(this, &SMediaPlayerEditorViewer::HandleTimerTextBlockText)
															.ToolTipText(this, &SMediaPlayerEditorViewer::HandleTimerTextBlockToolTipText)
													]

												// fps
												+ SHorizontalBox::Slot()
													.MinWidth(45.f)
													.Padding(8.0f, 0.0f, 4.0f, 0.0f)
													.VAlign(VAlign_Center)
													.HAlign(HAlign_Left)
													[
														SNew(STextBlock)
															.Text(this, &SMediaPlayerEditorViewer::HandleFpsTextBlockText)
													]

												// buffering indicator
												+ SHorizontalBox::Slot()
													.AutoWidth()
													.Padding(4.0f, 0.0f)
													.VAlign(VAlign_Center)
													[
														SNew(SThrobber)
															.ToolTipText(LOCTEXT("BufferingTooltip", "Buffering..."))
															.Visibility_Lambda([this]() -> EVisibility
															{
																return MediaPlayer->IsBuffering() ? EVisibility::Visible : EVisibility::Hidden;
															})
													]
											]


										// transport controls
										+ SHorizontalBox::Slot()
											.FillWidth(1.0f)
											.Padding(8.0f, 0.0f)
											.VAlign(VAlign_Center)
											.HAlign(HAlign_Center)
											[
												CreatePlayerControls()
											]

										// settings
										+ SHorizontalBox::Slot()
											.AutoWidth()
											.Padding(4.0f, 0.0f, 4.0f, 0.0f)
											.VAlign(VAlign_Center)
											[
												SNew(SComboButton)
													.ContentPadding(0.0f)
													.ButtonContent()
													[
														SNew(SHorizontalBox)

														+ SHorizontalBox::Slot()
															.AutoWidth()
															.VAlign(VAlign_Center)
															[
																SNew(SImage)
																	.Image(InStyle->GetBrush("MediaPlayerEditor.SettingsButton"))
															]

														+ SHorizontalBox::Slot()
															.AutoWidth()
															.Padding(3.0f, 0.0f, 0.0f, 0.0f)
															.VAlign(VAlign_Center)
															[
																SNew(STextBlock)
																	.Text(LOCTEXT("OptionsButton", "Playback Options"))
															]
													]
													.ButtonStyle(FAppStyle::Get(), "ToggleButton")
													.ForegroundColor(FSlateColor::UseForeground())
													.MenuContent()
													[
														SettingsMenuBuilder.MakeWidget()
													]
											]
									]
							]
					]
			]

		+ SOverlay::Slot()
			[
				// drag & drop indicator
				SNew(SBorder)
					.BorderImage(FAppStyle::GetBrush("WhiteBrush"))
					.BorderBackgroundColor_Lambda([this]() -> FLinearColor {
						return DragValid
							? FLinearColor(0.0f, 1.0f, 0.0f, 0.15f)
							: FLinearColor(1.0f, 0.0f, 0.0f, 0.15f);
					})
					.Visibility_Lambda([this]() -> EVisibility {
						return DragOver && FSlateApplication::Get().IsDragDropping()
							? EVisibility::HitTestInvisible
							: EVisibility::Hidden;
					})
			]
	];
}

UMediaPlayer* SMediaPlayerEditorViewer::GetMediaPlayer() const
{
	return MediaPlayer;
}

void SMediaPlayerEditorViewer::EnableMouseControl(bool bIsEnabled)
{
	if (PlayerViewport.IsValid())
	{
		PlayerViewport->EnableMouseControl(bIsEnabled);
	}
}

MediaPlayerEditor::MediaImage::ETextureChannelMask SMediaPlayerEditorViewer::GetChannelMask() const
{
	if (PlayerViewport.IsValid())
	{
		return PlayerViewport->GetChannelMask();
	}

	return MediaPlayerEditor::MediaImage::ETextureChannelMask::RGBA;
}

bool SMediaPlayerEditorViewer::IsChannelMasked(MediaPlayerEditor::MediaImage::ETextureChannelMask InChannel) const
{
	if (PlayerViewport.IsValid())
	{
		return EnumHasAllFlags(PlayerViewport->GetChannelMask(), InChannel);
	}

	return true;
}

void SMediaPlayerEditorViewer::SetChannelMask(MediaPlayerEditor::MediaImage::ETextureChannelMask InMask)
{
	if (PlayerViewport.IsValid())
	{
		PlayerViewport->SetChannelMask(InMask);
	}
}

void SMediaPlayerEditorViewer::ToggleChannelMask(MediaPlayerEditor::MediaImage::ETextureChannelMask InChannelToToggle)
{
	if (!PlayerViewport.IsValid())
	{
		return;
	}

	using namespace MediaPlayerEditor::MediaImage;

	const ETextureChannelMask CurrentMask = PlayerViewport->GetChannelMask();

	if (EnumHasAllFlags(CurrentMask, InChannelToToggle))
	{
		PlayerViewport->SetChannelMask(CurrentMask & ~InChannelToToggle);
	}
	else
	{
		PlayerViewport->SetChannelMask(CurrentMask | InChannelToToggle);
	}
}

/* SWidget interface
 *****************************************************************************/

void SMediaPlayerEditorViewer::OnDragEnter(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent)
{
	DragOver = true;

	TSharedPtr<FExternalDragOperation> DragDropOp = DragDropEvent.GetOperationAs<FExternalDragOperation>();
	DragValid = DragDropOp.IsValid() && DragDropOp->HasFiles();
}


void SMediaPlayerEditorViewer::OnDragLeave(const FDragDropEvent& DragDropEvent)
{
	DragOver = false;
}


FReply SMediaPlayerEditorViewer::OnDragOver(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent)
{
	if (DragValid)
	{
		return FReply::Handled();
	}

	return SCompoundWidget::OnDragOver(MyGeometry, DragDropEvent);
}


FReply SMediaPlayerEditorViewer::OnDrop(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent)
{
	TSharedPtr<FExternalDragOperation> DragDropOp = DragDropEvent.GetOperationAs<FExternalDragOperation>();

	if (DragDropOp.IsValid() && DragDropOp->HasFiles())
	{
		const TArray<FString>& Files = DragDropOp->GetFiles();

		if (Files.Num() > 0)
		{
			MediaPlayer->Close();

			for (int32 FileIndex = 0; FileIndex < Files.Num(); ++FileIndex)
			{
				const FString FilePath = FPaths::ConvertRelativePathToFull(Files[FileIndex]);
				MediaPlayer->GetPlaylist()->AddFile(FilePath);
			}

			MediaPlayer->Next();

			return FReply::Handled();
		}
	}

	return SCompoundWidget::OnDrop(MyGeometry,DragDropEvent);
}


/* SMediaPlayerEditorPlayer implementation
 *****************************************************************************/

void SMediaPlayerEditorViewer::MakeCaptureDeviceMenu(TArray<FMediaCaptureDeviceInfo>& DeviceInfos, FMenuBuilder& MenuBuilder)
{
	for (const FMediaCaptureDeviceInfo& DeviceInfo : DeviceInfos)
	{
		MenuBuilder.AddMenuEntry(
			DeviceInfo.DisplayName,
			FText::GetEmpty(),
			FSlateIcon(),
			FUIAction(FExecuteAction::CreateLambda([this, Url = DeviceInfo.Url] {
				MediaPlayer->OpenUrl(Url);
			})),
			NAME_None,
			EUserInterfaceActionType::Button
		);
	}
}


void SMediaPlayerEditorViewer::OpenUrl(const FText& TextUrl)
{
	LastUrl = TextUrl;

	FString Url = TextUrl.ToString();
	{
		Url.TrimStartAndEndInline();
	}

	if (!Url.Contains(TEXT("://"), ESearchCase::CaseSensitive))
	{
		Url.InsertAt(0, TEXT("file://"));
	}

	MediaPlayer->OpenUrl(Url);
}


void SMediaPlayerEditorViewer::SetDesiredPlayerName(FName PlayerName)
{
	if (PlayerName != MediaPlayer->GetDesiredPlayerName())
	{
		MediaPlayer->SetDesiredPlayerName(PlayerName);

		if ((PlayerName != NAME_None) && (PlayerName != MediaPlayer->GetPlayerName()))
		{
			MediaPlayer->Reopen();
		}
	}

	UMediaPlayerEditorSettings* Settings = GetMutableDefault<UMediaPlayerEditorSettings>();
	{
		Settings->DesiredPlayerName = PlayerName;
		Settings->SaveConfig();
	}
}


/* SMediaPlayerEditorPlayer callbacks
 *****************************************************************************/

void SMediaPlayerEditorViewer::HandleAudioCaptureDevicesMenuNewMenu(FMenuBuilder& MenuBuilder)
{
	TArray<FMediaCaptureDeviceInfo> DeviceInfos;
	MediaCaptureSupport::EnumerateAudioCaptureDevices(DeviceInfos);
	MakeCaptureDeviceMenu(DeviceInfos, MenuBuilder);
}


void SMediaPlayerEditorViewer::HandleDecoderMenuNewMenu(FMenuBuilder& MenuBuilder)
{
	// automatic player option
	MenuBuilder.AddMenuEntry(
		LOCTEXT("AutoPlayer", "Automatic"),
		LOCTEXT("AutoPlayerTooltip", "Select a player automatically based on the media source"),
		FSlateIcon(),
		FUIAction(
			FExecuteAction::CreateLambda([this] { SetDesiredPlayerName(NAME_None); }),
			FCanExecuteAction(),
			FIsActionChecked::CreateLambda([this] { return MediaPlayer->GetDesiredPlayerName() == NAME_None; })
		),
		NAME_None,
		EUserInterfaceActionType::RadioButton
	);

	MenuBuilder.AddMenuSeparator();

	// get registered player plug-ins
	const IMediaModule* MediaModule = FModuleManager::LoadModulePtr<IMediaModule>("Media");

	if (MediaModule == nullptr)
	{
		return;
	}

	TArray<IMediaPlayerFactory*> PlayerFactories = MediaModule->GetPlayerFactories();

	if (PlayerFactories.Num() == 0)
	{
		TSharedRef<SWidget> NoPlayersAvailableWidget = SNew(STextBlock)
			.ColorAndOpacity(FSlateColor::UseSubduedForeground())
			.Text(LOCTEXT("NoPlayerPluginsInstalled", "No media player plug-ins installed"));

		MenuBuilder.AddWidget(NoPlayersAvailableWidget, FText::GetEmpty(), true, false);

		return;
	}

	PlayerFactories.Sort([](IMediaPlayerFactory& A, IMediaPlayerFactory& B) -> bool {
		return (A.GetDisplayName().CompareTo(B.GetDisplayName()) < 0);
	});

	// add option for each player
	const FString PlatformName(FPlatformProperties::IniPlatformName());

	for (IMediaPlayerFactory* Factory : PlayerFactories)
	{
		const bool SupportsRunningPlatform = Factory->GetSupportedPlatforms().Contains(PlatformName);
		const FName PlayerName = Factory->GetPlayerName();

		MenuBuilder.AddMenuEntry(
			FText::Format(LOCTEXT("PlayerNameFormat", "{0} ({1})"), Factory->GetDisplayName(), FText::FromName(PlayerName)),
			FText::FromString(FString::Join(Factory->GetSupportedPlatforms(), TEXT(", "))),
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateLambda([this, PlayerName] { SetDesiredPlayerName(PlayerName); }),
				FCanExecuteAction::CreateLambda([SupportsRunningPlatform] { return SupportsRunningPlatform; }),
				FIsActionChecked::CreateLambda([this, PlayerName] { return MediaPlayer->GetDesiredPlayerName() == PlayerName; })
			),
			NAME_None,
			EUserInterfaceActionType::RadioButton
		);
	}
}


void SMediaPlayerEditorViewer::HandleFormatMenuNewMenu(FMenuBuilder& MenuBuilder, EMediaPlayerTrack TrackType)
{
	const int32 SelectedTrack = MediaPlayer->GetSelectedTrack(TrackType);
	const int32 NumFormats = MediaPlayer->GetNumTrackFormats(TrackType, SelectedTrack);

	if ((SelectedTrack != INDEX_NONE) && (NumFormats > 0))
	{
		for (int32 FormatIndex = 0; FormatIndex < NumFormats; ++FormatIndex)
		{
			FText DisplayText;

			if (TrackType == EMediaPlayerTrack::Audio)
			{
				const uint32 Channels = MediaPlayer->GetAudioTrackChannels(SelectedTrack, FormatIndex);
				const uint32 SampleRate = MediaPlayer->GetAudioTrackSampleRate(SelectedTrack, FormatIndex);
				const FString Type = MediaPlayer->GetAudioTrackType(SelectedTrack, FormatIndex);

				DisplayText = FText::Format(LOCTEXT("TrackFormatMenuAudioFormat", "{0}: {1} {2} channels @ {3} Hz"),
					FText::AsNumber(FormatIndex),
					FText::FromString(Type),
					FText::AsNumber(Channels),
					FText::AsNumber(SampleRate)
				);
			}
			else if (TrackType == EMediaPlayerTrack::Video)
			{
				const FIntPoint Dim = MediaPlayer->GetVideoTrackDimensions(SelectedTrack, FormatIndex);
				const float FrameRate = MediaPlayer->GetVideoTrackFrameRate(SelectedTrack, FormatIndex);
				const TRange<float> FrameRates = MediaPlayer->GetVideoTrackFrameRates(SelectedTrack, FormatIndex);
				const FString Type = MediaPlayer->GetVideoTrackType(SelectedTrack, FormatIndex);

				FFormatNamedArguments Arguments;
				Arguments.Add(TEXT("Index"), FText::AsNumber(FormatIndex));
				Arguments.Add(TEXT("DimX"), FText::AsNumber(Dim.X));
				Arguments.Add(TEXT("DimY"), FText::AsNumber(Dim.Y));
				Arguments.Add(TEXT("Fps"), FText::AsNumber(FrameRate));
				Arguments.Add(TEXT("Type"), FText::FromString(Type));

				if (FrameRates.IsDegenerate() && (FrameRates.GetLowerBoundValue() == FrameRate))
				{
					DisplayText = FText::Format(LOCTEXT("TrackFormatMenuVideoFormat", "{Index}: {Type} {DimX}x{DimY} {Fps} fps"), Arguments);
				}
				else
				{
					Arguments.Add(TEXT("FpsLower"), FText::AsNumber(FrameRates.GetLowerBoundValue()));
					Arguments.Add(TEXT("FpsUpper"), FText::AsNumber(FrameRates.GetUpperBoundValue()));

					DisplayText = FText::Format(LOCTEXT("TrackFormatMenuVideoFormat2", "{Index}: {Type} {DimX}x{DimY} {Fps} [{FpsLower}-{FpsUpper}] fps"), Arguments);
				}
			}
			else
			{
				DisplayText = LOCTEXT("TrackFormatDefault", "Default");
			}

			MenuBuilder.AddMenuEntry(
				DisplayText,
				FText::GetEmpty(),
				FSlateIcon(),
				FUIAction(
					FExecuteAction::CreateLambda([this, TrackType, SelectedTrack, FormatIndex] { MediaPlayer->SetTrackFormat(TrackType, SelectedTrack, FormatIndex); }),
					FCanExecuteAction(),
					FIsActionChecked::CreateLambda([this, TrackType, SelectedTrack, FormatIndex] { return (MediaPlayer->GetTrackFormat(TrackType, SelectedTrack) == FormatIndex); })
				),
				NAME_None,
				EUserInterfaceActionType::RadioButton
			);
		}
	}
	else
	{
		FText Label = (SelectedTrack == INDEX_NONE)
			? LOCTEXT("NoTrackSelectedLabel", "No track selected")
			: LOCTEXT("NoFormatsAvailableLabel", "No formats available");

		TSharedRef<SWidget> NoTracksAvailableWidget = SNew(SBox)
			.Padding(FMargin(2.0f, 2.0f, 2.0f, 0.0f))
			[
				SNew(STextBlock)
				.ColorAndOpacity(FSlateColor::UseSubduedForeground())
			.Text(Label)
			];

		MenuBuilder.AddWidget(NoTracksAvailableWidget, FText::GetEmpty(), true, false);
	}
}


FText SMediaPlayerEditorViewer::HandleFpsTextBlockText() const
{
	if (!MediaPlayer->IsReady())
	{
		return FText::GetEmpty();
	}

	const int32 SelectedTrack = MediaPlayer->GetSelectedTrack(EMediaPlayerTrack::Video);
	const int32 SelectedFormat = MediaPlayer->GetTrackFormat(EMediaPlayerTrack::Video, SelectedTrack);
	const float FrameRate = MediaPlayer->GetVideoTrackFrameRate(SelectedTrack, SelectedFormat);

	// empty string if fps n/a
	if (FrameRate <= 0.0f)
	{
		return FText::GetEmpty();
	}

	FNumberFormattingOptions FormattingOptions;
	FormattingOptions.MaximumFractionalDigits = 3;

	return FText::Format(LOCTEXT("FpsTextBlockFormat", "{0} fps"), FText::AsNumber(FrameRate, &FormattingOptions));
}


void SMediaPlayerEditorViewer::HandleMediaPlayerMediaEvent(EMediaEvent Event)
{
	if (Event == EMediaEvent::MediaOpened)
	{
		LastUrl = FText::GetEmpty();
	}
	else if (Event == EMediaEvent::MediaOpenFailed)
	{
		FNotificationInfo NotificationInfo(LOCTEXT("MediaOpenFailedError", "The media failed to open. Check Output Log for details!"));
		{
			NotificationInfo.ExpireDuration = 2.0f;
		}

		FSlateNotificationManager::Get().AddNotification(NotificationInfo)->SetCompletionState(SNotificationItem::CS_Fail);
	}
}


void SMediaPlayerEditorViewer::HandleScaleMenuNewMenu(FMenuBuilder& MenuBuilder)
{
	MenuBuilder.AddMenuEntry(
		LOCTEXT("ScaleFitMenuLabel", "Fit"),
		LOCTEXT("ScaleFitMenuTooltip", "Scale the video to fit the viewport, but maintain the aspect ratio"),
		FSlateIcon(),
		FUIAction(
			FExecuteAction::CreateLambda([]{
				UMediaPlayerEditorSettings* Settings = GetMutableDefault<UMediaPlayerEditorSettings>();
				Settings->ViewportScale = EMediaPlayerEditorScale::Fit;
				Settings->SaveConfig();
			}),
			FCanExecuteAction(),
			FIsActionChecked::CreateLambda([]{ return (GetDefault<UMediaPlayerEditorSettings>()->ViewportScale == EMediaPlayerEditorScale::Fit); })
		),
		NAME_None,
		EUserInterfaceActionType::RadioButton
	);

	MenuBuilder.AddMenuEntry(
		LOCTEXT("ScaleFillMenuLabel", "Fill"),
		LOCTEXT("ScaleFillMenuTooltip", "Scale the video non-uniformly to fill the entire viewport"),
		FSlateIcon(),
		FUIAction(
			FExecuteAction::CreateLambda([]{
				UMediaPlayerEditorSettings* Settings = GetMutableDefault<UMediaPlayerEditorSettings>();
				Settings->ViewportScale = EMediaPlayerEditorScale::Fill;
				Settings->SaveConfig();
			}),
			FCanExecuteAction(),
			FIsActionChecked::CreateLambda([]{ return (GetDefault<UMediaPlayerEditorSettings>()->ViewportScale == EMediaPlayerEditorScale::Fill); })
		),
		NAME_None,
		EUserInterfaceActionType::RadioButton
	);

	MenuBuilder.AddMenuEntry(
		LOCTEXT("ScaleOriginalMenuLabel", "Original Size"),
		LOCTEXT("ScaleOriginalMenuTooltip", "Do not scale or stretch the video"),
		FSlateIcon(),
		FUIAction(
			FExecuteAction::CreateLambda([]{
				UMediaPlayerEditorSettings* Settings = GetMutableDefault<UMediaPlayerEditorSettings>();
				Settings->ViewportScale = EMediaPlayerEditorScale::Original;
				Settings->SaveConfig();
			}),
			FCanExecuteAction(),
			FIsActionChecked::CreateLambda([]{ return (GetDefault<UMediaPlayerEditorSettings>()->ViewportScale == EMediaPlayerEditorScale::Original); })
		),
		NAME_None,
		EUserInterfaceActionType::RadioButton
	);
}

void SMediaPlayerEditorViewer::HandleTimerUnitMenuNewMenu(FMenuBuilder& MenuBuilder)
{
	MenuBuilder.AddMenuEntry(
		LOCTEXT("TimerUnitTimecodesMenuLabel", "Timecode (HH:MM:SS)"),
		LOCTEXT("TimerUnitTimecodesMenuTooltip", "Display the timer as HH:MM:SS formatted timecode."),
		FSlateIcon(),
		FUIAction(
			FExecuteAction::CreateLambda([]{
				UMediaPlayerEditorSettings* Settings = GetMutableDefault<UMediaPlayerEditorSettings>();
				Settings->TimerUnit = EMediaPlayerEditorTimerUnit::Timecode;
				Settings->SaveConfig();
			}),
			FCanExecuteAction(),
			FIsActionChecked::CreateLambda([]{ return (GetDefault<UMediaPlayerEditorSettings>()->TimerUnit == EMediaPlayerEditorTimerUnit::Timecode); })
		),
		NAME_None,
		EUserInterfaceActionType::RadioButton
	);

	MenuBuilder.AddMenuEntry(
		LOCTEXT("TimerUnitFramesMenuLabel", "Frame Number"),
		LOCTEXT("TimerUnitFramesMenuTooltip", "Display the timer as frame number. Note: will revert to \"timecode\" for variable frame rate videos."),
		FSlateIcon(),
		FUIAction(
			FExecuteAction::CreateLambda([]{
				UMediaPlayerEditorSettings* Settings = GetMutableDefault<UMediaPlayerEditorSettings>();
				Settings->TimerUnit = EMediaPlayerEditorTimerUnit::FrameNumber;
				Settings->SaveConfig();
			}),
			FCanExecuteAction(),
			FIsActionChecked::CreateLambda([]{ return (GetDefault<UMediaPlayerEditorSettings>()->TimerUnit == EMediaPlayerEditorTimerUnit::FrameNumber); })
		),
		NAME_None,
		EUserInterfaceActionType::RadioButton
	);
}

FText SMediaPlayerEditorViewer::HandleTimerTextBlockText() const
{
	if (!MediaPlayer->IsReady())
	{
		return FText::GetEmpty();
	}

	const FTimespan Time = MediaPlayer->GetDisplayTime();

	// empty string if time n/a
	if (Time < FTimespan::Zero())
	{
		return FText::GetEmpty();
	}
	
	const FTimespan Duration = MediaPlayer->GetDuration();

	const FText TimerTextBlockFormat = LOCTEXT("TimerTextBlockFormat", "{0} / {1}");

	if (GetDefault<UMediaPlayerEditorSettings>()->TimerUnit == EMediaPlayerEditorTimerUnit::FrameNumber)
	{
		const FTimespan FrameDuration = UE::MediaPlayerEditor::GetVideoFrameDuration(MediaPlayer);

		// Note: if the frame rate is not available, we fall back to displaying the time code. 
		if (FrameDuration != FTimespan::Zero())
		{
			const int32 FrameNumber = static_cast<int32>(FMath::RoundHalfToEven(Time.GetTotalSeconds() / FrameDuration.GetTotalSeconds()));
			
			// time only if duration n/a
			if (Duration <= FTimespan::Zero())
			{
				return FText::AsNumber(FrameNumber);
			}

			// format time & duration
			const FText DurationText = (Duration == FTimespan::MaxValue())
				? FText::FromString(TEXT("\u221E")) // infinity symbol
				: FText::AsNumber(static_cast<int32>(FMath::RoundHalfToEven(Duration.GetTotalSeconds() / FrameDuration.GetTotalSeconds())));

			return FText::Format(TimerTextBlockFormat, FText::AsNumber(FrameNumber), DurationText);
		}
	}

	// time only if duration n/a
	if (Duration <= FTimespan::Zero())
	{
		return FText::AsTimespan(Time);
	}

	// format time & duration
	const FText DurationText = (Duration == FTimespan::MaxValue())
		? FText::FromString(TEXT("\u221E")) // infinity symbol
		: FText::AsTimespan(Duration);

	return FText::Format(TimerTextBlockFormat, FText::AsTimespan(Time), DurationText);
}


FText SMediaPlayerEditorViewer::HandleTimerTextBlockToolTipText() const
{
	if (!MediaPlayer->IsReady())
	{
		return FText::GetEmpty();
	}

	FTimespan Duration = MediaPlayer->GetDuration();
	FTimespan Remaining = Duration - MediaPlayer->GetDisplayTime();
	bool bInfiniteTimeRemaining = Duration == FTimespan::MaxValue();

	if (Remaining <= FTimespan::Zero() && !bInfiniteTimeRemaining)
	{
		return LOCTEXT("UnknownTimeRemainingTooltip", "Unknown time remaining");
	}

	if (Remaining == FTimespan::MaxValue() || bInfiniteTimeRemaining)
	{
		return LOCTEXT("InfiniteTimeRemainingTooltip", "Infinite time remaining");
	}

	return FText::Format(LOCTEXT("TimeRemainingTooltipFormat", "{0} remaining"), FText::AsTimespan(Remaining));
}

void SMediaPlayerEditorViewer::HandleBackgroundMenuNewMenu(FMenuBuilder& MenuBuilder)
{
	MenuBuilder.AddMenuEntry(
		LOCTEXT("BackgroundBlack", "Black"),
		LOCTEXT("BackgroundBlackTooltip", "Use a black media background"),
		FSlateIcon(),
		FUIAction(
			FExecuteAction::CreateLambda(
				[] 
				{
					UMediaPlayerEditorSettings* Settings = GetMutableDefault<UMediaPlayerEditorSettings>();
					Settings->MediaBackground = EMediaPlayerEditorBackground::Black;
					Settings->SaveConfig();
				}),
			FCanExecuteAction(),
			FIsActionChecked::CreateLambda(
				[] 
				{ 
					return GetDefault<UMediaPlayerEditorSettings>()->MediaBackground == EMediaPlayerEditorBackground::Black;
				})
		),
		NAME_None,
		EUserInterfaceActionType::RadioButton
	);

	MenuBuilder.AddMenuEntry(
		LOCTEXT("BackgroundCheckered", "Checkered"),
		LOCTEXT("BackgroundCheckeredTooltip", "Use a checkered media background"),
		FSlateIcon(),
		FUIAction(
			FExecuteAction::CreateLambda(
				[] 
				{
					UMediaPlayerEditorSettings* Settings = GetMutableDefault<UMediaPlayerEditorSettings>();
					Settings->MediaBackground = EMediaPlayerEditorBackground::Checkered;
					Settings->SaveConfig();
				}),
			FCanExecuteAction(),
			FIsActionChecked::CreateLambda(
				[] 
				{ 
					return GetDefault<UMediaPlayerEditorSettings>()->MediaBackground == EMediaPlayerEditorBackground::Checkered;
				})
		),
		NAME_None,
		EUserInterfaceActionType::RadioButton
	);
}


void SMediaPlayerEditorViewer::HandleTrackMenuNewMenu(FMenuBuilder& MenuBuilder, EMediaPlayerTrack TrackType)
{
	const int32 NumTracks = MediaPlayer->GetNumTracks(TrackType);

	if (NumTracks > 0)
	{
		MenuBuilder.AddMenuEntry(
			LOCTEXT("DisabledTrackMenuName", "Disabled"),
			FText::GetEmpty(),
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateLambda([this, TrackType] { MediaPlayer->SelectTrack(TrackType, INDEX_NONE); }),
				FCanExecuteAction(),
				FIsActionChecked::CreateLambda([this, TrackType] { return (MediaPlayer->GetSelectedTrack(TrackType) == INDEX_NONE); })
			),
			NAME_None,
			EUserInterfaceActionType::RadioButton
		);

		MenuBuilder.AddMenuSeparator();

		FInternationalization& I18n = FInternationalization::Get();

		for (int32 TrackIndex = 0; TrackIndex < NumTracks; ++TrackIndex)
		{
			const FText DisplayName = MediaPlayer->GetTrackDisplayName(TrackType, TrackIndex);
			const FString Language = MediaPlayer->GetTrackLanguage(TrackType, TrackIndex);
			const FCulturePtr Culture = I18n.GetCulture(Language);
			const FString LanguageDisplayName = Culture.IsValid() ? Culture->GetDisplayName() : FString();
			const FString LanguageNativeName = Culture.IsValid() ? Culture->GetNativeName() : FString();
			
			const FText DisplayText = LanguageNativeName.IsEmpty() ? DisplayName : FText::Format(LOCTEXT("TrackNameFormat", "{0} ({1})"), DisplayName, FText::FromString(LanguageNativeName));
			const FText TooltipText = FText::FromString(LanguageDisplayName);

			MenuBuilder.AddMenuEntry(
				DisplayText,
				TooltipText,
				FSlateIcon(),
				FUIAction(
					FExecuteAction::CreateLambda([this, TrackType, TrackIndex] { MediaPlayer->SelectTrack(TrackType, TrackIndex); }),
					FCanExecuteAction(),
					FIsActionChecked::CreateLambda([this, TrackType, TrackIndex] { return (MediaPlayer->GetSelectedTrack(TrackType) == TrackIndex); })
				),
				NAME_None,
				EUserInterfaceActionType::RadioButton
			);
		}
	}
	else
	{
		TSharedRef<SWidget> NoTracksAvailableWidget = SNew(SBox)
			.Padding(FMargin(2.0f, 2.0f, 2.0f, 0.0f))
			[
				SNew(STextBlock)
					.ColorAndOpacity(FSlateColor::UseSubduedForeground())
					.Text(LOCTEXT("NoTracksAvailableLabel", "No tracks available"))
			];

		MenuBuilder.AddWidget(NoTracksAvailableWidget, FText::GetEmpty(), true, false);
	}
}


FReply SMediaPlayerEditorViewer::HandleUrlBoxKeyDown(const FGeometry&, const FKeyEvent& KeyEvent)
{
	if (KeyEvent.GetKey() == EKeys::Escape)
	{
		UrlTextBox->SetText(LastUrl);

		return FReply::Handled().ClearUserFocus(true);
	}

	return FReply::Unhandled();
}


void SMediaPlayerEditorViewer::HandleVideoCaptureDevicesMenuNewMenu(FMenuBuilder& MenuBuilder)
{
	TArray<FMediaCaptureDeviceInfo> DeviceInfos;
	MediaCaptureSupport::EnumerateVideoCaptureDevices(DeviceInfos);
	MakeCaptureDeviceMenu(DeviceInfos, MenuBuilder);
}

TSharedRef<SWidget> SMediaPlayerEditorViewer::CreatePlayerControls()
{
	return SNew(SMediaControls, CommandListWeak.Pin(), MediaPlayer);
}

#undef LOCTEXT_NAMESPACE
