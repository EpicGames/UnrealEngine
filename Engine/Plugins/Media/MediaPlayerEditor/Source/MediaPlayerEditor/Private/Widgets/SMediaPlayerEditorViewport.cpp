// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/SMediaPlayerEditorViewport.h"

#include "Framework/Commands/UICommandList.h"
#include "MediaPlayer.h"
#include "Models/MediaPlayerEditorCommands.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SOverlay.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Layout/SScaleBox.h"
#include "Widgets/Text/STextBlock.h"

#include "Shared/MediaPlayerEditorSettings.h"
#include "SMediaPlayerEditorOutput.h"
#include "SMediaPlayerEditorOverlay.h"


#define LOCTEXT_NAMESPACE "SMediaPlayerEditorViewport"


/* SMediaPlayerEditorPlayer structors
 *****************************************************************************/

SMediaPlayerEditorViewport::SMediaPlayerEditorViewport()
	: MediaPlayer(nullptr)
	, bIsMouseControlEnabled(true)
{ }


/* SMediaPlayerEditorPlayer interface
 *****************************************************************************/

void SMediaPlayerEditorViewport::Construct(const FArguments& InArgs, UMediaPlayer& InMediaPlayer,
	UMediaTexture* InMediaTexture, const TSharedRef<ISlateStyle>& InStyle,
	bool bInIsSoundEnabled, TSharedPtr<FUICommandList> InCommands)
{
	MediaPlayer = &InMediaPlayer;
	Style = InStyle;
	CommandsWeak = InCommands;

	const auto OverlayFont = Style->GetFontStyle("MediaPlayerEditor.ViewportFont");

	ChildSlot
	[
		SNew(SOverlay)

		+ SOverlay::Slot()
			[
				SNew(SBorder)
				.BorderImage(this, &SMediaPlayerEditorViewport::GetMediaBorderImage)
				.Padding(0.0f)
				[
					// movie viewport
					SNew(SScaleBox)
						.Stretch_Lambda([]() -> EStretch::Type {
							auto Settings = GetDefault<UMediaPlayerEditorSettings>();
							if (Settings->ViewportScale == EMediaPlayerEditorScale::Fill)
							{
								return EStretch::Fill;
							}
							if (Settings->ViewportScale == EMediaPlayerEditorScale::Fit)
							{
								return EStretch::ScaleToFit;
							}
							return EStretch::None;
						})
						[
							// movie texture
							SAssignNew(Output, SMediaPlayerEditorOutput, InMediaPlayer, InMediaTexture, bInIsSoundEnabled)
						]
					]
				]

		+ SOverlay::Slot()
			[
				// subtitle & caption overlays
				SNew(SMediaPlayerEditorOverlay, InMediaPlayer)
					.Visibility_Lambda([]{ return GetDefault<UMediaPlayerEditorSettings>()->ShowTextOverlays ? EVisibility::Visible : EVisibility::Collapsed; })
			]

		+ SOverlay::Slot()
			.Padding(FMargin(12.0f, 8.0f))
			[
				// info overlays
				SNew(SVerticalBox)
					.Visibility_Lambda([this]() -> EVisibility {
						return (IsHovered() || HasMouseCapture()) ? EVisibility::Visible : EVisibility::Hidden;
					})

				+ SVerticalBox::Slot()
					.VAlign(VAlign_Top)
					[
						SNew(SHorizontalBox)

						+ SHorizontalBox::Slot()
							.HAlign(HAlign_Left)
							.VAlign(VAlign_Top)
							[
								// media source name
								SNew(STextBlock)
									.ColorAndOpacity(FSlateColor::UseSubduedForeground())
									.Font(OverlayFont)
									.ShadowOffset(FVector2D(1.f, 1.f))
									.Text(this, &SMediaPlayerEditorViewport::HandleMediaSourceNameText)
									.ToolTipText(LOCTEXT("OverlaySourceNameTooltip", "Name of the currently opened media source"))
							]

						+ SHorizontalBox::Slot()
							.HAlign(HAlign_Right)
							.VAlign(VAlign_Top)
							[
								// player name
								SNew(STextBlock)
									.ColorAndOpacity(FSlateColor::UseSubduedForeground())
									.Font(OverlayFont)
									.ShadowOffset(FVector2D(1.f, 1.f))
									.Text(this, &SMediaPlayerEditorViewport::HandleMediaPlayerNameText)
									.ToolTipText(LOCTEXT("OverlayPlayerNameTooltip", "Name of the currently used media player plug-in"))
							]
					]

				+ SVerticalBox::Slot()
					.VAlign(VAlign_Bottom)
					[
						SNew(SHorizontalBox)

						+ SHorizontalBox::Slot()
							.HAlign(HAlign_Left)
							.VAlign(VAlign_Bottom)
							[
								// playback state
								SNew(STextBlock)
									.ColorAndOpacity(FSlateColor::UseSubduedForeground())
									.Font(OverlayFont)
									.ShadowOffset(FVector2D(1.f, 1.f))
									.Text(this, &SMediaPlayerEditorViewport::HandleMediaPlayerStateText)
									.ToolTipText(LOCTEXT("OverlayPlayerStateTooltip", "The media player's current state"))
							]
					]
			]

		+ SOverlay::Slot()
			.Padding(FMargin(12.0f, 8.0f))
			.HAlign(HAlign_Center)
			.VAlign(VAlign_Center)
			[
				// notification overlay
				SNew(STextBlock)
					.ColorAndOpacity(FSlateColor::UseSubduedForeground())
					.Font(OverlayFont)
					.ShadowOffset(FVector2D(1.f, 1.f))
					.Text(this, &SMediaPlayerEditorViewport::HandleNotificationText)
			]
	];
}

MediaPlayerEditor::MediaImage::ETextureChannelMask SMediaPlayerEditorViewport::GetChannelMask() const
{
	if (Output.IsValid())
	{
		return Output->GetChannelMask();
	}

	return MediaPlayerEditor::MediaImage::ETextureChannelMask::RGBA;
}

void SMediaPlayerEditorViewport::SetChannelMask(MediaPlayerEditor::MediaImage::ETextureChannelMask InMask)
{
	if (Output.IsValid())
	{
		Output->SetChannelMask(InMask);
	}
}


/* SWidget interface
 *****************************************************************************/

FReply SMediaPlayerEditorViewport::OnMouseButtonDoubleClick(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	if (TSharedPtr<FUICommandList> Commands = CommandsWeak.Pin())
	{
		Commands->TryExecuteAction(FMediaPlayerEditorCommands::Get().TogglePlayPauseMedia.ToSharedRef());
	}

	return FReply::Handled();
}


FReply SMediaPlayerEditorViewport::OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent)
{
	if (TSharedPtr<FUICommandList> Commands = CommandsWeak.Pin())
	{
		if (Commands->ProcessCommandBindings(InKeyEvent))
		{
			return FReply::Handled();
		}
	}

	return FReply::Unhandled();
}


/* SMediaPlayerEditorViewport callbacks
 *****************************************************************************/

FText SMediaPlayerEditorViewport::HandleMediaPlayerNameText() const
{
	FName PlayerName = MediaPlayer->GetPlayerName();

	if ((PlayerName == NAME_None) || MediaPlayer->GetUrl().IsEmpty())
	{
		const FName DesiredPlayerName = MediaPlayer->GetDesiredPlayerName();

		if (DesiredPlayerName == NAME_None)
		{
			return LOCTEXT("AutoPlayerName", "Auto");
		}

		return FText::FromName(DesiredPlayerName);
	}

	return FText::FromName(PlayerName);
}


FText SMediaPlayerEditorViewport::HandleMediaPlayerStateText() const
{
	if (MediaPlayer->HasError())
	{
		return LOCTEXT("StateOverlayError", "Error");
	}

	if (MediaPlayer->IsBuffering())
	{
		return LOCTEXT("StateOverlayBuffering", "Buffering");
	}

	if (MediaPlayer->IsPreparing())
	{
		return LOCTEXT("StateOverlayPreparing", "Preparing");
	}

	if (!MediaPlayer->IsReady())
	{
		return LOCTEXT("StateOverlayStopped", "Not Ready");
	}

	if (MediaPlayer->IsPaused())
	{
		return LOCTEXT("StateOverlayPaused", "Paused");
	}

	if (MediaPlayer->IsPlaying())
	{
		float Rate = MediaPlayer->GetRate();

		if (Rate == 1.0f)
		{
			return LOCTEXT("StateOverlayPlaying", "Playing");
		}

		if (Rate < 0.0f)
		{
			return FText::Format(LOCTEXT("StateOverlayReverseFormat", "Reverse ({0}x)"), FText::AsNumber(-Rate));
		}

		return FText::Format(LOCTEXT("StateOverlayForwardFormat", "Forward ({0}x)"), FText::AsNumber(Rate));
	}

	return LOCTEXT("StateOverlayReady", "Ready");
}


FText SMediaPlayerEditorViewport::HandleMediaSourceNameText() const
{
	const FText MediaName = MediaPlayer->GetMediaName();

	if (MediaName.IsEmpty())
	{
		return LOCTEXT("StateOverlayNoMedia", "No Media");
	}

	return MediaName;
}


FText SMediaPlayerEditorViewport::HandleNotificationText() const
{
	if (MediaPlayer->IsPlaying())
	{
		if (MediaPlayer->GetNumTracks(EMediaPlayerTrack::Video) == 0)
		{
			return LOCTEXT("NoVideoTrackAvailable", "No video track available");
		}

		const int32 SelectedVideoTrack = MediaPlayer->GetSelectedTrack(EMediaPlayerTrack::Video);

		if (SelectedVideoTrack == INDEX_NONE)
		{
			return LOCTEXT("NoVideoTrackSelected", "No video track selected");
		}

		if (MediaPlayer->GetNumTrackFormats(EMediaPlayerTrack::Video, SelectedVideoTrack) == 0)
		{
			return LOCTEXT("NoVideoFormatsAvailable", "No video formats available");
		}

		if (MediaPlayer->GetTrackFormat(EMediaPlayerTrack::Video, SelectedVideoTrack) == INDEX_NONE)
		{
			return LOCTEXT("NoVideoFormatSelected", "No video format selected");
		}
	}

	return FText::GetEmpty();
}

const FSlateBrush* SMediaPlayerEditorViewport::GetMediaBorderImage() const
{
	if (const UMediaPlayerEditorSettings* Settings = GetDefault<UMediaPlayerEditorSettings>())
	{
		switch (Settings->MediaBackground)
		{
			case EMediaPlayerEditorBackground::Checkered:
				return FCoreStyle::Get().GetBrush("ColorPicker.AlphaBackground");

			default:
				// Fall back to default black background
				break;
		}
	}

	return FCoreStyle::Get().GetBrush("BlackBrush");
}


#undef LOCTEXT_NAMESPACE
