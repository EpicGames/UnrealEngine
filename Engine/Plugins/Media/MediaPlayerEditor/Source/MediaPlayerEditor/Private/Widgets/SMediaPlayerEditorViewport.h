// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Styling/ISlateStyle.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/SMediaImage.h"

class FUICommandList;
class SMediaPlayerEditorOutput;
class UMediaPlayer;
class UMediaSoundComponent;
class UMediaTexture;
struct FSlateBrush;

class SMediaPlayerEditorViewport
	: public SCompoundWidget
{
public:

	SLATE_BEGIN_ARGS(SMediaPlayerEditorViewport) { }
	SLATE_END_ARGS()

public:

	/** Default constructor. */
	SMediaPlayerEditorViewport();

public:

	/**
	 * Construct this widget
	 *
	 * @param InArgs The declaration data for this widget.
	 * @param InMediaPlayer The UMediaPlayer asset to show the details for.
	 * @param InMediaTexture The UMediaTexture asset to output video to. If nullptr then use our own.
	 * @param InStyle The style set to use.
	 * @param bInIsSoundEnabled If true then produce sound.
	 * @param InCommands Command list for keybinds.
	 */
	void Construct(const FArguments& InArgs, UMediaPlayer& InMediaPlayer,
		UMediaTexture* InMediaTexture, const TSharedRef<ISlateStyle>& InStyle,
		bool bInIsSoundEnabled, TSharedPtr<FUICommandList> InCommands);

	/**
	 * Enables/disables using the mouse to control the viewport.
	 */
	void EnableMouseControl(bool bIsEnabled) { bIsMouseControlEnabled = bIsEnabled; }

	/**
	 * Returns the active mask.
	 */
	MediaPlayerEditor::MediaImage::ETextureChannelMask GetChannelMask() const;

	/**
	 * Sets a channel-based mask for the image.
	 *
	 * If only a single channel is displayed, it will show in greyscale.
	 *
	 * @param InMask to display.
	 */
	void SetChannelMask(MediaPlayerEditor::MediaImage::ETextureChannelMask InMask);

public:

	//~ SWidget interface

	virtual FReply OnMouseButtonDoubleClick(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	virtual FReply OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent) override;

private:

	/** Callback for getting the text of the player name overlay. */
	FText HandleMediaPlayerNameText() const;

	/** Callback for getting the text of the playback state overlay. */
	FText HandleMediaPlayerStateText() const;

	/** Callback for getting the text of the media source name overlay. */
	FText HandleMediaSourceNameText() const;

	/** Callback for getting the text of the notification overlay. */
	FText HandleNotificationText() const;

	/** Returns the image used for the background of the media. */
	const FSlateBrush* GetMediaBorderImage() const;

private:

	/** Pointer to the media player that is being viewed. */
	UMediaPlayer* MediaPlayer;

	/** The style set to use for this widget. */
	TSharedPtr<ISlateStyle> Style;
	
	/** True if the mouse can control things. */
	bool bIsMouseControlEnabled;

	/** The output for the media player. */
	TSharedPtr<SMediaPlayerEditorOutput> Output;

	/** Command list for executing commands. */
	TWeakPtr<FUICommandList> CommandsWeak;
};