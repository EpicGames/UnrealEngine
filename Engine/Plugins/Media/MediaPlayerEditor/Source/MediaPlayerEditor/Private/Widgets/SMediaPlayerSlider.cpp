// Copyright Epic Games, Inc. All Rights Reserved.

#include "SMediaPlayerSlider.h"
#include "MediaPlayer.h"
#include "SlateOptMacros.h"
#include "Widgets/Input/SSlider.h"

BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION
void SMediaPlayerSlider::Construct(const FArguments& InArgs, const TArrayView<TWeakObjectPtr<UMediaPlayer>> InMediaPlayers)
{
	MediaPlayerEntries.Reserve(InMediaPlayers.Num());
	for (const TWeakObjectPtr<UMediaPlayer>& MediaPlayerWeak : InMediaPlayers)
	{
		if (MediaPlayerWeak.IsValid())
		{
			MediaPlayerEntries.Emplace(MediaPlayerWeak);
		}
	}

	ChildSlot
	[
		SAssignNew(ScrubberSlider, SSlider)
		.IsEnabled_Raw(this, &SMediaPlayerSlider::DoesMediaPlayerSupportSeeking)
		.OnMouseCaptureBegin_Raw(this, &SMediaPlayerSlider::OnScrubBegin)
		.OnMouseCaptureEnd_Raw(this, &SMediaPlayerSlider::OnScrubEnd)
		.OnValueChanged_Raw(this, &SMediaPlayerSlider::Seek)
		.Value_Raw(this, &SMediaPlayerSlider::GetPlaybackPosition)
		.Visibility_Raw(this, &SMediaPlayerSlider::GetScrubberVisibility)
		.Orientation(Orient_Horizontal)
		.SliderBarColor(FLinearColor::Transparent)
		.Style(InArgs._Style)
		.PreventThrottling(true)
	];
}
END_SLATE_FUNCTION_BUILD_OPTIMIZATION

bool SMediaPlayerSlider::DoesMediaPlayerSupportSeeking() const
{
	// Return true if at least one player supports seek.
	for (const FMediaPlayerEntry& Entry : MediaPlayerEntries)
	{
		if (const UMediaPlayer* MediaPlayer = Entry.MediaPlayerWeak.Get())
		{
			if (MediaPlayer->SupportsSeeking())
			{
				return true;
			}
		}
	}
	return false;
}

void SMediaPlayerSlider::OnScrubBegin()
{
	TArray<UMediaPlayer*, TInlineAllocator<1>> MediaPlayers;
	MediaPlayers.Reserve(MediaPlayerEntries.Num());

	for (FMediaPlayerEntry& Entry : MediaPlayerEntries)
	{
		if (UMediaPlayer* MediaPlayer = Entry.MediaPlayerWeak.Get())
		{
			MediaPlayers.Add(MediaPlayer);

			Entry.ScrubValue = static_cast<float>(FTimespan::Ratio(MediaPlayer->GetDisplayTime(), MediaPlayer->GetDuration()));
			Entry.LastScrubValue = Entry.ScrubValue;

			if (MediaPlayer->SupportsScrubbing())
			{
				Entry.PreScrubRate = MediaPlayer->GetRate();
				MediaPlayer->SetRate(0.0f);
			}
		}
	}

	ScrubEvent.Broadcast(IMediaPlayerSlider::EScrubEventType::Begin, MediaPlayers, GetSliderValue());
}

void SMediaPlayerSlider::OnScrubEnd()
{
	TArray<UMediaPlayer*, TInlineAllocator<1>> MediaPlayers;
	MediaPlayers.Reserve(MediaPlayerEntries.Num());

	for (FMediaPlayerEntry& Entry : MediaPlayerEntries)
	{
		if (UMediaPlayer* MediaPlayer = Entry.MediaPlayerWeak.Get())
		{
			MediaPlayers.Add(MediaPlayer);

			// Set playback position to scrub value when drag ends
			if (Entry.LastScrubValue != Entry.ScrubValue && MediaPlayer->SupportsSeeking())
			{
				MediaPlayer->Seek(MediaPlayer->GetDuration() * Entry.ScrubValue);
			}

			if (MediaPlayer->SupportsScrubbing())
			{
				MediaPlayer->SetRate(Entry.PreScrubRate);
			}
		}
	}

	ScrubEvent.Broadcast(IMediaPlayerSlider::EScrubEventType::End, MediaPlayers, GetSliderValue());
}

void SMediaPlayerSlider::Seek(float InPlaybackPosition)
{
	TArray<UMediaPlayer*, TInlineAllocator<1>> MediaPlayers;
	MediaPlayers.Reserve(MediaPlayerEntries.Num());

	for (FMediaPlayerEntry& Entry : MediaPlayerEntries)
	{
		if (UMediaPlayer* MediaPlayer = Entry.MediaPlayerWeak.Get())
		{
			MediaPlayers.Add(MediaPlayer);

			Entry.ScrubValue = InPlaybackPosition;

			if (!ScrubberSlider->HasMouseCapture() || MediaPlayer->SupportsScrubbing())
			{
				MediaPlayer->Scrub(MediaPlayer->GetDuration() * InPlaybackPosition);
				Entry.LastScrubValue = Entry.ScrubValue;
			}
		}
	}

	ScrubEvent.Broadcast(IMediaPlayerSlider::EScrubEventType::Update, MediaPlayers, InPlaybackPosition);
}

float SMediaPlayerSlider::GetPlaybackPosition() const
{
	// All scrub positions should match, so search for the first valid player. 
	// Give priority to players with a video track.
	const FMediaPlayerEntry* PlayerEntry = FindValidPlayerEntryForTrackType(EMediaPlayerTrack::Video);
	if (!PlayerEntry)
	{
		// Fallback to audio track second (not all players support it).
		PlayerEntry = FindValidPlayerEntryForTrackType(EMediaPlayerTrack::Audio);
	}

	if (PlayerEntry)
	{
		if (ScrubberSlider->HasMouseCapture())
		{
			return PlayerEntry->ScrubValue;
		}

		if (const UMediaPlayer* MediaPlayer = PlayerEntry->MediaPlayerWeak.Get())
		{
			return static_cast<float>(FTimespan::Ratio(MediaPlayer->GetDisplayTime(), MediaPlayer->GetDuration()));
		}
	}

	return 0.0f;
}

EVisibility SMediaPlayerSlider::GetScrubberVisibility() const
{
	bool bIsActive = false;
	for (const FMediaPlayerEntry& Entry : MediaPlayerEntries)
	{
		if (const UMediaPlayer* MediaPlayer = Entry.MediaPlayerWeak.Get())
		{
			bIsActive = (MediaPlayer->SupportsScrubbing() || MediaPlayer->SupportsSeeking());
			if (bIsActive)
			{
				break; // If any player is active, consider widget active.
			}
		}
	}

	return bIsActive ? EVisibility::Visible : VisibilityWhenInactive;
}

float SMediaPlayerSlider::GetSliderValue() const
{
	if (ScrubberSlider.IsValid())
	{
		return ScrubberSlider->GetValue();
	}

	return 0.f;
}

void SMediaPlayerSlider::SetSliderHandleColor(const FSlateColor& InSliderColor)
{
	if (ScrubberSlider)
	{
		ScrubberSlider->SetSliderHandleColor(InSliderColor);
	}
}

void SMediaPlayerSlider::SetSliderBarColor(const FSlateColor& InSliderColor)
{
	if (ScrubberSlider)
	{
		ScrubberSlider->SetSliderBarColor(InSliderColor);
	}
}

void SMediaPlayerSlider::SetVisibleWhenInactive(EVisibility InVisibility)
{
	VisibilityWhenInactive = InVisibility;
}

IMediaPlayerSlider::FScrubEvent::RegistrationType& SMediaPlayerSlider::GetScrubEvent()
{
	return ScrubEvent;
}

/** Find a valid player entry for the given track type. */
const SMediaPlayerSlider::FMediaPlayerEntry* SMediaPlayerSlider::FindValidPlayerEntryForTrackType(EMediaPlayerTrack InTrackType) const
{
	for (const FMediaPlayerEntry& Entry : MediaPlayerEntries)
	{
		if (const UMediaPlayer* MediaPlayer = Entry.MediaPlayerWeak.Get())
		{
			if (MediaPlayer->GetNumTracks(InTrackType) > 0 && MediaPlayer->GetDuration() > FTimespan::Zero())
			{
				return &Entry;
			}
		}
	}
	return nullptr;
}