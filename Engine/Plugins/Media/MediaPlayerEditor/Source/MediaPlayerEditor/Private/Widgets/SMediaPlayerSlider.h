// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MediaPlayerEditorModule.h"
#include "Styling/CoreStyle.h"
#include "Styling/SlateTypes.h"
#include "Styling/SlateWidgetStyleAsset.h"
#include "Widgets/SCompoundWidget.h"

class SSlider;
class UMediaPlayer;
enum class EMediaPlayerTrack : uint8;

/**
 * Implements a scrubber to visualize the current playback position of a Media Player
 * and interact with it.
 */
class SMediaPlayerSlider : public IMediaPlayerSlider
{
public:
	SLATE_BEGIN_ARGS(SMediaPlayerSlider)
			: _Style(&FCoreStyle::Get().GetWidgetStyle<FSliderStyle>("Slider"))
		{
		}

	/** The Slider style used to draw the scrubber. */
	SLATE_STYLE_ARGUMENT(FSliderStyle, Style)
	SLATE_END_ARGS()

	/** Constructs this widget with InArgs */
	void Construct(const FArguments& InArgs, const TArrayView<TWeakObjectPtr<UMediaPlayer>> InMediaPlayers);

	//~Begin IMediaPlayerSlider
	virtual void SetSliderHandleColor(const FSlateColor& InSliderColor) override;
	virtual void SetSliderBarColor(const FSlateColor& InSliderColor) override;
	virtual void SetVisibleWhenInactive(EVisibility InVisibility) override;
	virtual FScrubEvent::RegistrationType& GetScrubEvent() override;
	//~End IMediaPlayerSlider

private:
	bool DoesMediaPlayerSupportSeeking() const;
	void OnScrubBegin();
	void OnScrubEnd();
	void Seek(float InPlaybackPosition);
	float GetPlaybackPosition() const;
	EVisibility GetScrubberVisibility() const;
	float GetSliderValue() const;

	/** The scrubber visibility when inactive. */
	EVisibility VisibilityWhenInactive = EVisibility::Hidden;

	struct FMediaPlayerEntry
	{
		/** Pointer to the media players that is are viewed. */
		TWeakObjectPtr<UMediaPlayer> MediaPlayerWeak;
		
		/** The playback rate prior to scrubbing. */
		float PreScrubRate = 0.0f;
		/** The value currently being scrubbed to. */
		float ScrubValue = 0.0f;
		
		/** The last value set with media player while scrubbing. */
		float LastScrubValue = -1.0f;

		FMediaPlayerEntry(const TWeakObjectPtr<UMediaPlayer>& InMediaPlayerWeak) : MediaPlayerWeak(InMediaPlayerWeak) { }
	};

	const FMediaPlayerEntry* FindValidPlayerEntryForTrackType(EMediaPlayerTrack InTrackType) const;
	
	TArray<FMediaPlayerEntry> MediaPlayerEntries;
	
	/** Holds the scrubber slider. */
	TSharedPtr<SSlider> ScrubberSlider;

	/** Event subscribers can use to detect scrubbing. */
	IMediaPlayerSlider::FScrubEvent ScrubEvent;
};
