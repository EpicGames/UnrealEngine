// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/SMediaPlaylistEditorDetails.h"
#include "Modules/ModuleManager.h"
#include "PropertyEditorModule.h"
#include "IDetailsView.h"
#include "MediaPlaylist.h"


#define LOCTEXT_NAMESPACE "SMediaPlaylistEditorDetails"


/* SMediaPlaylistEditorDetails interface
 *****************************************************************************/

void SMediaPlaylistEditorDetails::Construct(const FArguments& InArgs, UMediaPlaylist& InMediaPlaylist, const TSharedRef<ISlateStyle>& InStyle)
{
	MediaPlaylist = &InMediaPlaylist;

	// initialize details view
	FDetailsViewArgs DetailsViewArgs;
	{
		DetailsViewArgs.bAllowSearch = true;
		DetailsViewArgs.bHideSelectionTip = true;
		DetailsViewArgs.bLockable = false;
		DetailsViewArgs.bSearchInitialKeyFocus = true;
		DetailsViewArgs.bUpdatesFromSelection = false;
		DetailsViewArgs.bShowOptions = true;
		DetailsViewArgs.bShowModifiedPropertiesOption = false;
	}

	TSharedPtr<IDetailsView> DetailsView = FModuleManager::GetModuleChecked<FPropertyEditorModule>("PropertyEditor").CreateDetailView(DetailsViewArgs);
	DetailsView->SetObject(MediaPlaylist);

	ChildSlot
	[
		DetailsView.ToSharedRef()
	];
}


#undef LOCTEXT_NAMESPACE
