// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Styling/ISlateStyle.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"

class UMediaPlaylist;

/**
 * Implements the details panel of the MediaPlaylist asset editor.
 */
class SMediaPlaylistEditorDetails
	: public SCompoundWidget
{
public:

	SLATE_BEGIN_ARGS(SMediaPlaylistEditorDetails) { }
	SLATE_END_ARGS()

public:

	/**
	 * Construct this widget
	 *
	 * @param InArgs The declaration data for this widget.
	 * @param InMediaPlaylist The MediaPlaylist asset to show the details for.
	 * @param InStyleSet The style set to use.
	 */
	void Construct(const FArguments& InArgs, UMediaPlaylist& InMediaPlaylist, const TSharedRef<ISlateStyle>& InStyle);

private:

	/** Pointer to the MediaPlaylist asset that is being viewed. */
	UMediaPlaylist* MediaPlaylist;
};
