// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/SMediaPlaylistEditorMedia.h"
#include "Modules/ModuleManager.h"
#include "IContentBrowserSingleton.h"
#include "ContentBrowserModule.h"
#include "Widgets/Layout/SBorder.h"
#include "Styling/AppStyle.h"
#include "ScopedTransaction.h"
#include "MediaSource.h"
#include "MediaPlaylist.h"


#define LOCTEXT_NAMESPACE "SMediaPlaylistEditorMedia"


/* SMediaPlaylistEditorDetails interface
 *****************************************************************************/

void SMediaPlaylistEditorMedia::Construct(const FArguments& InArgs, UMediaPlaylist& InMediaPlaylist, const TSharedRef<ISlateStyle>& InStyle)
{
	MediaPlaylist = &InMediaPlaylist;

	// initialize asset picker
	FAssetPickerConfig AssetPickerConfig;
	{
		AssetPickerConfig.Filter.ClassPaths.Add(UMediaSource::StaticClass()->GetClassPathName());
		AssetPickerConfig.Filter.bRecursiveClasses = true;
		AssetPickerConfig.bAllowDragging = false;
		AssetPickerConfig.bAutohideSearchBar = true;
		AssetPickerConfig.bCanShowClasses = false;
		AssetPickerConfig.bCanShowDevelopersFolder = true;
		AssetPickerConfig.InitialAssetViewType = EAssetViewType::Column;
		AssetPickerConfig.ThumbnailScale = 0.1f;

		AssetPickerConfig.OnAssetDoubleClicked = FOnAssetDoubleClicked::CreateSP(this, &SMediaPlaylistEditorMedia::HandleAssetPickerAssetDoubleClicked);
	}

	auto& ContentBrowserModule = FModuleManager::Get().LoadModuleChecked<FContentBrowserModule>(TEXT("ContentBrowser"));

	ChildSlot
	[
		SNew(SBorder)
			.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
			.ToolTipText(LOCTEXT("DoubleClickToAddToolTip", "Double-click a media source to add it to the play list."))
			[
				ContentBrowserModule.Get().CreateAssetPicker(AssetPickerConfig)
			]
	];
}

/* SMediaPlaylistEditorDetails callbacks
 *****************************************************************************/

void SMediaPlaylistEditorMedia::HandleAssetPickerAssetDoubleClicked(const struct FAssetData& AssetData)
{
	UMediaSource* MediaSource = Cast<UMediaSource>(AssetData.GetAsset());
	
	if (MediaSource != nullptr)
	{
		FScopedTransaction Transaction(LOCTEXT("AddMediaSourceTransaction", "Add Media Source to Playlist"));

		MediaPlaylist->PreEditChange(nullptr);
		MediaPlaylist->Add(MediaSource);
		MediaPlaylist->PostEditChange();
	}
}


#undef LOCTEXT_NAMESPACE
