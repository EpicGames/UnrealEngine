// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Styling/ISlateStyle.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"

class UMediaPlaylist;

/**
 * Implements the media library of the MediaPlaylist asset editor.
 */
class SMediaPlaylistEditorMedia
	: public SCompoundWidget
{
public:

	SLATE_BEGIN_ARGS(SMediaPlaylistEditorMedia) { }
	SLATE_END_ARGS()

public:

	/**
	 * Construct this widget
	 *
	 * @param InArgs The declaration data for this widget.
	 * @param InMediaPlaylist The MediaPlaylist asset to show the details for.
	 * @param InStyleSet The style set to use.
	 */
	void Construct(const FArguments& InArgs, UMediaPlaylist& InMediaPlaylist, const TSharedRef<ISlateStyle>& InStyle);

private:

	/** Callback for double-clicking an asset in the asset picker. */
	void HandleAssetPickerAssetDoubleClicked(const struct FAssetData& AssetData);

private:

	/** Pointer to the MediaPlaylist asset that is being viewed. */
	UMediaPlaylist* MediaPlaylist;
};
