// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/SMediaPlaylistEditorTracks.h"
#include "AssetRegistry/AssetData.h"
#include "Modules/ModuleManager.h"
#include "PropertyCustomizationHelpers.h"
#include "PropertyEditorModule.h"
#include "IDetailsView.h"
#include "MediaPlaylist.h"
#include "MediaSource.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Layout/SScrollBox.h"
#include "Widgets/Text/STextBlock.h"
#include "Styling/AppStyle.h"
#include "ScopedTransaction.h"
#include "UObject/UObjectGlobals.h"

#define LOCTEXT_NAMESPACE "SMediaPlaylistEditorTracks"

void SMediaPlaylistEditorTracks::Construct(const FArguments& InArgs, UMediaPlaylist* InMediaPlaylist, const TSharedRef<ISlateStyle>& InStyle)
{
	MediaPlaylistPtr = InMediaPlaylist;
	bShowAddEntryButton = InArgs._ShowAddEntryButton;

	ChildSlot
	.Padding(4.0f)
		[
			SNew(SScrollBox)

			// Buttons to manipulate playlist.
			+ SScrollBox::Slot()
				.Padding(2)
				.VAlign(VAlign_Center)
				[
					SNew(SHorizontalBox)

					+ SHorizontalBox::Slot()
						.AutoWidth()
						.Padding(2)
						.HAlign(HAlign_Center)
						[
							// Add button to add to the playlist.
							SNew(SButton)
							.ToolTipText(LOCTEXT("Add_ToolTip", "Add an entry to the playlist."))
							.VAlign(VAlign_Center)
							.Visibility(this, &SMediaPlaylistEditorTracks::GetAddEntryButtonVisibility)
							.OnClicked_Lambda([this]() -> FReply
							{
								AddToPlaylist();
								return FReply::Handled();
							})
							[
								SNew(SImage)
									.ColorAndOpacity(FSlateColor::UseForeground())
									.Image(FAppStyle::GetBrush("Icons.PlusCircle"))
							]
						]
				]

			// Container for our media sources.
			+ SScrollBox::Slot()
				[
					SNew(SVerticalBox)
					+ SVerticalBox::Slot()
					.AutoHeight()
					[
						SNew(STextBlock)
						.Visibility(this, &SMediaPlaylistEditorTracks::GetSourcesContainerHeaderTextVisibility)
						.Text(this, &SMediaPlaylistEditorTracks::GetSourcesContainerHeaderText)
					]
					+ SVerticalBox::Slot()
					.AutoHeight()
					[
						SAssignNew(SourcesContainer, SVerticalBox)
					]
				]
		];

	FCoreUObjectDelegates::OnObjectTransacted.AddSP(this, &SMediaPlaylistEditorTracks::OnObjectTransacted);
	RefreshPlaylist();
}

SMediaPlaylistEditorTracks::~SMediaPlaylistEditorTracks()
{
	// AddSP is technically safe from dangling pointers but let's be nice and remove ourselves.
	FCoreUObjectDelegates::OnObjectTransacted.RemoveAll(this);
}

void SMediaPlaylistEditorTracks::RefreshPlaylist()
{
	SourcesContainer->ClearChildren();

	UMediaPlaylist* MediaPlaylist = MediaPlaylistPtr.Get();
	if (MediaPlaylist != nullptr)
	{
		// Add each item in the playlist.
		for (int32 Index = 0; Index < MediaPlaylist->Num(); ++Index)
		{
			UMediaSource* MediaSource = MediaPlaylist->Get(Index);

			SourcesContainer->AddSlot()
				.AutoHeight()
				.Padding(2)
				.VAlign(VAlign_Center)
				[
					SNew(SHorizontalBox)
					
					+ SHorizontalBox::Slot()
						.AutoWidth()
						.Padding(2)
						.HAlign(HAlign_Left)
						[
							// Add asset picker.
							SNew(SObjectPropertyEntryBox)
								.AllowedClass(UMediaSource::StaticClass())
								.ObjectPath(this, &SMediaPlaylistEditorTracks::GetMediaSourcePath, Index)
								.OnObjectChanged(this, &SMediaPlaylistEditorTracks::OnMediaSourceChanged, Index)
						]

					+ SHorizontalBox::Slot()
						.AutoWidth()
						.Padding(2)
						.HAlign(HAlign_Left)
						[
							PropertyCustomizationHelpers::MakeInsertDeleteDuplicateButton(
								FExecuteAction::CreateLambda([this, Index]()
								{
									UMediaPlaylist* MediaPlaylist = MediaPlaylistPtr.Get();
									if (MediaPlaylist != nullptr)
									{
#if WITH_EDITOR
                                    	FScopedTransaction Transaction(LOCTEXT("Array.InsertMediaSource", "Insert Media Source"));
                                    	MediaPlaylist->Modify();
#endif
										MediaPlaylist->Insert(nullptr, Index);
										RefreshPlaylist();
									}
								}),
								FExecuteAction::CreateLambda([this, Index]()
								{
									UMediaPlaylist* MediaPlaylist = MediaPlaylistPtr.Get();
									if (MediaPlaylist != nullptr)
									{
#if WITH_EDITOR
										FScopedTransaction Transaction(LOCTEXT("Array.RemoveMediaSource", "Remove Media Source"));
										MediaPlaylist->Modify();
#endif
										MediaPlaylist->RemoveAt(Index);
										RefreshPlaylist();
									}
								}),
								FExecuteAction::CreateLambda([this, Index]()
								{
									UMediaPlaylist* MediaPlaylist = MediaPlaylistPtr.Get();
									if (MediaPlaylist != nullptr)
									{
#if WITH_EDITOR
										FScopedTransaction Transaction(LOCTEXT("Array.DuplicateMediaSource", "Duplicate Media Source"));
										MediaPlaylist->Modify();
#endif
										MediaPlaylist->Insert(MediaPlaylist->Get(Index), Index);
										RefreshPlaylist();
									}
								}))
						]
				];
		}
	}
}

void SMediaPlaylistEditorTracks::AddToPlaylist()
{
	UMediaPlaylist* MediaPlaylist = MediaPlaylistPtr.Get();
	if (MediaPlaylist != nullptr)
	{
#if WITH_EDITOR
		FScopedTransaction Transaction(LOCTEXT("AddToPlaylist", "Add Media Source"));
		MediaPlaylist->Modify();
#endif
		MediaPlaylist->Insert(nullptr, MediaPlaylist->Num());
		RefreshPlaylist();
	}
}

FString SMediaPlaylistEditorTracks::GetMediaSourcePath(int32 Index) const
{
	FString Path;

	UMediaPlaylist* MediaPlaylist = MediaPlaylistPtr.Get();
	if (MediaPlaylist != nullptr)
	{
		UMediaSource* MediaSource = MediaPlaylist->Get(Index);
		if (MediaSource != nullptr)
		{
			Path = MediaSource->GetPathName();
		}
	}

	return Path;
}

void SMediaPlaylistEditorTracks::OnMediaSourceChanged(const FAssetData& AssetData, int32 Index)
{
	UMediaPlaylist* MediaPlaylist = MediaPlaylistPtr.Get();
	if (MediaPlaylist != nullptr)
	{
		UMediaSource* MediaSource = Cast<UMediaSource>(AssetData.GetAsset());
#if WITH_EDITOR
		FScopedTransaction Transaction(LOCTEXT("ChangeMediaSource", "Change Media Source"));
		MediaPlaylist->Modify();
#endif	
		MediaPlaylist->Replace(Index, MediaSource);
	}
}

void SMediaPlaylistEditorTracks::OnObjectTransacted(UObject* Object, const FTransactionObjectEvent& TransactionObjectEvent)
{
	if (Object == MediaPlaylistPtr.Get())
	{
		RefreshPlaylist();
	}
}

EVisibility SMediaPlaylistEditorTracks::GetAddEntryButtonVisibility() const
{
	return bShowAddEntryButton && MediaPlaylistPtr.IsValid() ? EVisibility::Visible : EVisibility::Collapsed;
}

EVisibility SMediaPlaylistEditorTracks::GetSourcesContainerHeaderTextVisibility() const
{
	TStrongObjectPtr<UMediaPlaylist> MediaPlaylistPinned = MediaPlaylistPtr.Pin();
	if (!MediaPlaylistPinned)
	{
		return EVisibility::Visible;
	}

	if (MediaPlaylistPinned->Num() == 0)
	{
		return EVisibility::Visible;
	}

	return EVisibility::Collapsed;
}

FText SMediaPlaylistEditorTracks::GetSourcesContainerHeaderText() const
{
	TStrongObjectPtr<UMediaPlaylist> MediaPlaylistPinned = MediaPlaylistPtr.Pin();
	if (!MediaPlaylistPinned)
	{
		return LOCTEXT("MissingPlaylistText", "No playlist source selected.");
	}

	if (MediaPlaylistPinned->Num() == 0)
	{
		return LOCTEXT("EmptyPlaylistText", "Selected playlist source is empty.");
	}

	return FText();
}

#undef LOCTEXT_NAMESPACE
