// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/SMediaSourceEditorDetails.h"

#include "IDetailsView.h"
#include "MediaSource.h"
#include "Modules/ModuleManager.h"
#include "PropertyEditorModule.h"

#define LOCTEXT_NAMESPACE "SMediaSourceEditorDetails"

void SMediaSourceEditorDetails::Construct(const FArguments& InArgs, UMediaSource& InMediaSource, const TSharedRef<ISlateStyle>& InStyle)
{
	MediaSource = &InMediaSource;

	// Initialize details view
	FDetailsViewArgs DetailsViewArgs;
	{
		DetailsViewArgs.bAllowSearch = true;
		DetailsViewArgs.bHideSelectionTip = true;
		DetailsViewArgs.bLockable = false;
		DetailsViewArgs.bSearchInitialKeyFocus = true;
		DetailsViewArgs.bUpdatesFromSelection = false;
		DetailsViewArgs.bShowOptions = true;
		DetailsViewArgs.bShowModifiedPropertiesOption = false;
	}

	TSharedPtr<IDetailsView> DetailsView = FModuleManager::GetModuleChecked<FPropertyEditorModule>("PropertyEditor").CreateDetailView(DetailsViewArgs);
	DetailsView->SetObject(MediaSource);

	ChildSlot
	[
		DetailsView.ToSharedRef()
	];
}


#undef LOCTEXT_NAMESPACE
