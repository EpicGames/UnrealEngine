// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Styling/ISlateStyle.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"

class UMediaSource;

/**
 * Implements the details panel of the MediaSource asset editor.
 */
class SMediaSourceEditorDetails
	: public SCompoundWidget
{
public:

	SLATE_BEGIN_ARGS(SMediaSourceEditorDetails) { }
	SLATE_END_ARGS()

public:

	/**
	 * Construct this widget.
	 *
	 * @param InArgs		The declaration data for this widget.
	 * @param InMediaSource	The MediaSource to show the details for.
	 * @param InStyleSet	The style set to use.
	 */
	void Construct(const FArguments& InArgs, UMediaSource& InMediaSource, const TSharedRef<ISlateStyle>& InStyle);

private:

	/** Pointer to the MediaPlate that is being viewed. */
	TObjectPtr<UMediaSource> MediaSource;
};
