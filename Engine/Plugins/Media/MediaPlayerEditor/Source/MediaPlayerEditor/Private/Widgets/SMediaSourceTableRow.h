// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Misc/Attribute.h"
#include "Widgets/SNullWidget.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SWidget.h"
#include "Widgets/Views/STableViewBase.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Views/STableRow.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Views/SListView.h"
#include "MediaSource.h"
#include "SlateOptMacros.h"
#include "Widgets/Images/SImage.h"

#define LOCTEXT_NAMESPACE "SMediaSourceTableRow"

struct FMediaSourceTableEntry
{
	/** The media source's index in the table. */
	int32 Index;

	/** The media source. */
	TWeakObjectPtr<UMediaSource> MediaSource;

	/** Create and initialize a new instance. */
	FMediaSourceTableEntry(int32 InIndex, UMediaSource* InMediaSource)
		: Index(InIndex)
		, MediaSource(InMediaSource)
	{ }
};


/**
 * Implements a row widget in a media source list.
 */
class SMediaSourceTableRow
	: public SMultiColumnTableRow<TSharedPtr<FMediaSourceTableEntry>>
{
public:

	SLATE_BEGIN_ARGS(SMediaSourceTableRow) { }
		SLATE_ARGUMENT(TSharedPtr<FMediaSourceTableEntry>, Entry)
		SLATE_ATTRIBUTE(bool, Opened)
		SLATE_ARGUMENT(TSharedPtr<ISlateStyle>, Style)
	SLATE_END_ARGS()

public:

	/**
	 * Constructs the widget.
	 *
	 * @param InArgs The construction arguments.
	 * @param InOwnerTableView The table view that owns this row.
	 */
	void Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& InOwnerTableView)
	{
		check(InArgs._Entry.IsValid());
		check(InArgs._Style.IsValid());

		Entry = InArgs._Entry;
		Opened = InArgs._Opened;
		Style = InArgs._Style;

		SMultiColumnTableRow<TSharedPtr<FMediaSourceTableEntry>>::Construct(FSuperRowType::FArguments(), InOwnerTableView);
	}

public:

	//~ SMultiColumnTableRow interface

	BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION
	virtual TSharedRef<SWidget> GenerateWidgetForColumn(const FName& ColumnName) override
	{
		if (ColumnName == "Icon")
		{
			if (Opened.Get(false))
			{
				return SNew(SBox)
					.HAlign(HAlign_Center)
					.VAlign(VAlign_Center)
					[	
						SNew(SImage)
							.Image(Style->GetBrush("MediaPlayerEditor.MediaSourceOpened"))
					];
			}
		}
		else if (ColumnName == "Index")
		{
			return SNew(STextBlock)
				.Text(FText::AsNumber(Entry->Index));
		}
		else if (ColumnName == "Source")
		{
			if (Entry->MediaSource.IsValid())
			{
				return SNew(STextBlock)
					.Text(FText::FromName(Entry->MediaSource->GetFName()));
			}
		}
		else if (ColumnName == "Type")
		{
			if (Entry->MediaSource.IsValid())
			{
				return SNew(STextBlock)
					.Text(Entry->MediaSource->GetClass()->GetDisplayNameText());
			}
		}

		return SNullWidget::NullWidget;
	}
	END_SLATE_FUNCTION_BUILD_OPTIMIZATION

private:

	/** The table entry being shown in this row. */
	TSharedPtr<FMediaSourceTableEntry> Entry;

	/** Whether the media source shown in this row is currently opened in a media player. */
	TAttribute<bool> Opened;

	/** The widget's visual style. */
	TSharedPtr<ISlateStyle> Style;
};


#undef LOCTEXT_NAMESPACE
