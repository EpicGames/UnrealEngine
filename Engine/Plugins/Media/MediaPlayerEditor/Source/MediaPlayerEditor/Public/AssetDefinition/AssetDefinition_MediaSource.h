// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "AssetDefinitionDefault.h"
#include "MediaSource.h"
#include "AssetDefinition_MediaSource.generated.h"

UCLASS(MinimalAPI)
class UAssetDefinition_MediaSource : public UAssetDefinitionDefault
{
	GENERATED_BODY()

public:
	// UAssetDefinition Begin
	virtual FText GetAssetDisplayName() const override { return FText::GetEmpty(); }
	virtual FLinearColor GetAssetColor() const override { return FColor::White; }
	virtual TSoftClassPtr<UObject> GetAssetClass() const override { return UMediaSource::StaticClass(); }
	MEDIAPLAYEREDITOR_API virtual EAssetCommandResult OpenAssets(const FAssetOpenArgs& OpenArgs) const override;
	// UAssetDefinition End
};
