// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Widgets/SWidget.h"
#include "AssetTypeActions_Base.h"

#define UE_API MEDIAPLAYEREDITOR_API

class ISlateStyle; 
struct FAssetData;

/**
 * Implements an action for UMediaSource assets.
 */
class FMediaSourceActions
	: public FAssetTypeActions_Base
{
public:

	//~ FAssetTypeActions_Base interface

	UE_API virtual bool CanFilter() override;
	UE_API virtual FText GetAssetDescription(const FAssetData& AssetData) const override;
	UE_API virtual uint32 GetCategories() override;
	UE_API virtual FText GetName() const override;
	UE_API virtual UClass* GetSupportedClass() const override;
	UE_API virtual FColor GetTypeColor() const override;
	UE_API virtual TSharedPtr<class SWidget> GetThumbnailOverlay(const FAssetData& AssetData) const override;

	UE_API virtual void OpenAssetEditor(const TArray<UObject*>& InObjects, TSharedPtr<IToolkitHost> EditWithinLevelEditor = TSharedPtr<IToolkitHost>()) override;
};

#undef UE_API
