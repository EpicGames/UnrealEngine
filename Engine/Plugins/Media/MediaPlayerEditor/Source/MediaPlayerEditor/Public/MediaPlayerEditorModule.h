// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Modules/ModuleInterface.h"
#include "Styling/CoreStyle.h"
#include "Styling/SlateTypes.h"
#include "Templates/SharedPointer.h"
#include "Widgets/SCompoundWidget.h"

class ISlateStyle;
class UMediaPlayer;

/**
 * Interface for MediaPlayer Playback Scrubber Widget
 */
class IMediaPlayerSlider : public SCompoundWidget
{
public:
	/** Set the Scrubber Slider Handle Color */
	virtual void SetSliderHandleColor(const FSlateColor& InSliderColor) = 0;

	/** Set the Scrubber Slider Color */
	virtual void SetSliderBarColor(const FSlateColor& InSliderColor) = 0;

	/** Set the Scrubber Slider Visibility, when the player is inactive */
	virtual void SetVisibleWhenInactive(EVisibility InVisibility) = 0;

	/** When scrubbing of the created slider occurs, this enum is used in FScrubEvent to inform subscribers about the slider state. */
	enum class EScrubEventType : uint8
	{
		/** Scrubbing has started. */
		Begin,
		/** Scrubbing is ongoing and the position of the scrub has changed. */
		Update,
		/** Scrubbing has stopped. */
		End
	};

	DECLARE_MULTICAST_DELEGATE_ThreeParams(FScrubEvent, EScrubEventType /* Event type */, TConstArrayView<UMediaPlayer*> /* Media Players */, float /* Slider/Scrub Value */)

	/** Event subscribers can use to detect scrubbing. */
	virtual FScrubEvent::RegistrationType& GetScrubEvent() = 0;
};

/**
* Interface for the MediaPlayerEditor module.
*/
class IMediaPlayerEditorModule
	: public IModuleInterface
{
public:

	/** Get the style used by this module. */
	virtual TSharedPtr<ISlateStyle> GetStyle() = 0;

	/**
	 * Creates a Widget to visualize playback time and scrub the content played by a Media Player
	 * @param InMediaPlayer: the player affected by the widget
	 * @param InStyle: the style chosen for this slider widget
	 * @return Scrubber Widget
	 */
	UE_DEPRECATED(5.5, "Use version with TArrayView instead")
	virtual TSharedRef<IMediaPlayerSlider> CreateMediaPlayerSliderWidget(UMediaPlayer* InMediaPlayer, const FSliderStyle& InStyle = FCoreStyle::Get().GetWidgetStyle<FSliderStyle>("Slider")) = 0;

	/**
	 * Creates a Widget to visualize playback time and scrub the content played by Media Players
	 * @param InMediaPlayers: the players affected by the widget
	 * @param InStyle: the style chosen for this slider widget
	 * @return Scrubber Widget
	 */
	virtual TSharedRef<IMediaPlayerSlider> CreateMediaPlayerSliderWidget(const TArrayView<TWeakObjectPtr<UMediaPlayer>> InMediaPlayers, const FSliderStyle& InStyle = FCoreStyle::Get().GetWidgetStyle<FSliderStyle>("Slider")) = 0;
};
