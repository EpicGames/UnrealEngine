// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Framework/Commands/Commands.h"

#define UE_API MEDIAPLAYEREDITOR_API

/**
 * Holds the UI commands for the MediaPlayerEditorToolkit widget.
 */
class FMediaPlayerEditorCommands
	: public TCommands<FMediaPlayerEditorCommands>
{
public:
	UE_API static const FMediaPlayerEditorCommands& GetExternal();

	/** Default constructor. */
	FMediaPlayerEditorCommands() 
		: TCommands<FMediaPlayerEditorCommands>("MediaPlayerEditor", NSLOCTEXT("Contexts", "MediaPlayerEditor", "MediaPlayer Editor"), NAME_None, "MediaPlayerEditorStyle")
	{ }

public:

	//~ TCommands interface

	virtual void RegisterCommands() override;
	
public:

	/** Close the currently opened media. */
	TSharedPtr<FUICommandInfo> CloseMedia;

	/** Plays media forward at ever increasing speed. */
	TSharedPtr<FUICommandInfo> ForwardMedia;

	/** Generate a thumnbnail. */
	TSharedPtr<FUICommandInfo> GenerateThumbnail;

	/** Jump to next item in the play list. */
	TSharedPtr<FUICommandInfo> NextMedia;

	/** Open the current media. */
	TSharedPtr<FUICommandInfo> OpenMedia;

	/** Pauses media playback. */
	TSharedPtr<FUICommandInfo> PauseMedia;

	/** Starts media playback. */
	TSharedPtr<FUICommandInfo> PlayMedia;

	/** Starts media playback in reverse. */
	TSharedPtr<FUICommandInfo> PlayReverseMedia;

	/** Toggles between media play and pause. */
	TSharedPtr<FUICommandInfo> TogglePlayPauseMedia;

	/** Toggles between media reverse play and pause. */
	TSharedPtr<FUICommandInfo> TogglePlayReversePauseMedia;

	/** Jump to previous item in the play list. */
	TSharedPtr<FUICommandInfo> PreviousMedia;

	/** Reverses media playback at ever increasing speed. */
	TSharedPtr<FUICommandInfo> ReverseMedia;

	/** Rewinds the media to the beginning. */
	TSharedPtr<FUICommandInfo> RewindMedia;

	/** Fast forwards the media to the end. */
	TSharedPtr<FUICommandInfo> JumpToEndMedia;

	/** Step forward media 1 frame */
	TSharedPtr<FUICommandInfo> StepForwardMedia;

	/** Step backward media 1 frame. */
	TSharedPtr<FUICommandInfo> StepBackwardMedia;

	/** Toggles red texture channel. */
	TSharedPtr<FUICommandInfo> ToggledRedTextureChannel;

	/** Toggles green texture channel. */
	TSharedPtr<FUICommandInfo> ToggledGreenTextureChannel;

	/** Toggles blue texture channel. */
	TSharedPtr<FUICommandInfo> ToggledBlueTextureChannel;

	/** Toggles alpha texture channel. */
	TSharedPtr<FUICommandInfo> ToggledAlphaTextureChannel;
};

#undef UE_API
