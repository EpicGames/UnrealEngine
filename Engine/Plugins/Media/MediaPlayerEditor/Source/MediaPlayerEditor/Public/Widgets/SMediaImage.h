// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/GCObject.h"

#include "UObject/StrongObjectPtr.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"

#define UE_API MEDIAPLAYEREDITOR_API

class UMaterial;
class UMaterialInstanceDynamic;
class UTexture;

namespace MediaPlayerEditor
{
	namespace MediaImage
	{
		enum class ETextureChannelMask : uint8
		{
			None  = 0,
			Red   = 1 << 0,
			Green = 1 << 1,
			Blue  = 1 << 2,
			Alpha = 1 << 3,
			RGB   = Red | Green | Blue,
			RGBA  = Red | Green | Blue | Alpha,
		};
		ENUM_CLASS_FLAGS(ETextureChannelMask);
	}
}

class SMediaImage : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SMediaImage) { }
	SLATE_ATTRIBUTE(FVector2D, BrushImageSize);
	SLATE_END_ARGS()

public:

	/** Default constructor. */
	UE_API SMediaImage();

public:

	/**
	 * Construct this widget
	 *
	 * @param InArgs The declaration data for this widget.
	 * @param InTexture The source texture to render if any
	 */
	UE_API void Construct(const FArguments& InArgs, UTexture* InTexture);

	/**
	 * Tick this widget
	 * 
	 * @param InAllottedGeometry Geometry of the widget.
	 * @param InCurrentTime CurrentTime of the engine.
	 * @param InDeltaTime Deltatime since last frame.
	 */
	UE_API virtual void Tick(const FGeometry& InAllottedGeometry, const double InCurrentTime, const float InDeltaTime) override;

	/**
	 * Returns the active mask.
	 */
	UE_API MediaPlayerEditor::MediaImage::ETextureChannelMask GetChannelMask() const;

	/**
	 * Sets a channel-based mask for the image.
	 * 
	 * If only a single channel is displayed, it will show in greyscale.
	 * 
	 * @param InMask to display.
	 */
	UE_API void SetChannelMask(MediaPlayerEditor::MediaImage::ETextureChannelMask InMask);

private:
	static TMap<MediaPlayerEditor::MediaImage::ETextureChannelMask, TStrongObjectPtr<UMaterial>> CachedMaterials;

	/** The Slate brush that renders the material and its nodes. */
	TStrongObjectPtr<UMaterialInstanceDynamic> MaterialAInstance;

	/** The brush used to render the media. */
	TSharedPtr<FSlateBrush> MaterialBrush;

	/** Brush image size attribute. */
	TAttribute<FVector2D> BrushImageSize;

	/** The texture being displayed */
	TWeakObjectPtr<UTexture> TextureWeak;

	/** The active mask */
	MediaPlayerEditor::MediaImage::ETextureChannelMask Mask;
};

#undef UE_API
