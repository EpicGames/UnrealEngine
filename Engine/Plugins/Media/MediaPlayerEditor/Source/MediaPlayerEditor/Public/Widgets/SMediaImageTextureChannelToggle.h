// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/SCompoundWidget.h"

#include "Widgets/SMediaImage.h"

class FUICommandList;
class SMediaPlayerEditorViewer;
enum class ECheckBoxState : uint8;

#define UE_API MEDIAPLAYEREDITOR_API

/**
 * Implements a button to toggle the contents of a texture channel for the SMediaPlayerEditorViewer
 */
class SMediaImageTextureChannelToggle
	: public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SMediaImageTextureChannelToggle)
		{
		}
	SLATE_END_ARGS()

	/**
	 * Construct this widget
	 *
	 * @param InArgs The declaration data for this widget.
	 */
	UE_API void Construct(const FArguments& InArgs, TSharedPtr<SMediaPlayerEditorViewer> InViewer,
		MediaPlayerEditor::MediaImage::ETextureChannelMask InChannel, TSharedPtr<FUICommandList> InCommandList);

private:
	TWeakPtr<SMediaPlayerEditorViewer> ViewerWeak;
	MediaPlayerEditor::MediaImage::ETextureChannelMask Channel;
	TWeakPtr<FUICommandList> CommandListWeak;

	FSlateColor GetChannelButtonBackgroundColor() const;

	FSlateColor GetChannelButtonForegroundColor() const;

	ECheckBoxState OnGetChannelButtonCheckState() const;

	bool IsChannelButtonEnabled() const;

	void OnChannelButtonCheckStateChanged(ECheckBoxState InNewState);
};

#undef UE_API
