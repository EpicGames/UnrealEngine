// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Styling/ISlateStyle.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"

#define UE_API MEDIAPLAYEREDITOR_API

class STextBlock;
class UMediaPlayer;
class UMediaTexture;;

/**
 * Implements the media details panel of the MediaPlayer asset editor.
 */
class SMediaPlayerEditorMediaDetails
	: public SCompoundWidget
{
public:

	SLATE_BEGIN_ARGS(SMediaPlayerEditorMediaDetails) { }
	SLATE_END_ARGS()

public:

	/**
	 * Construct this widget.
	 *
	 * @param InArgs			The declaration data for this widget.
	 * @param InMediaPlayer		The MediaPlayer to show the details for.
	 * @param InMediaTexture	The MediaTexture to show the details for.
	 * @param InTextStyleName	The text style name to use.
	 *
	 */
	UE_API void Construct(const FArguments& InArgs, UMediaPlayer* InMediaPlayer, UMediaTexture* InMediaTexture, FName InTextStyleName = NAME_None);

	//~ SWidget interface
	UE_API virtual void Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime) override;

protected:
	/** Accessor for media player. */
	UE_API virtual UMediaPlayer* GetMediaPlayer() const;

	/** Accessor for the media texture. */
	UE_API virtual UMediaTexture* GetMediaTexture() const;

private:

	/**
	 * Updates our widgets to reflect the current state.
	 */
	void UpdateDetails();

	/** Pointer to the MediaPlayer that is being viewed. */
	TWeakObjectPtr<UMediaPlayer> MediaPlayerWeak;
	/** Pointer to the MediaTexture that is being viewed. */
	TWeakObjectPtr<UMediaTexture> MediaTextureWeak;

	/** Our widgets. */
	TSharedPtr<STextBlock> MediaPlayerName;
	TSharedPtr<STextBlock> FormatText;
	TSharedPtr<STextBlock> FrameRateText;
	TSharedPtr<STextBlock> LODBiasText;
	TSharedPtr<STextBlock> MethodText;
	TSharedPtr<STextBlock> NumMipsText;
	TSharedPtr<STextBlock> NumTilesText;
	TSharedPtr<STextBlock> ResolutionText;
	TSharedPtr<STextBlock> ResourceSizeText;
	TSharedPtr<STextBlock> StartTimecodeText;
	TSharedPtr<STextBlock> SeekPerformance;
	TSharedPtr<STextBlock> VideoDecoderProvider;
	TSharedPtr<STextBlock> AudioDecoderProvider;
};

#undef UE_API