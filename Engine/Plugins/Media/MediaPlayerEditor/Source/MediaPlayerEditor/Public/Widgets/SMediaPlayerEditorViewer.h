// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Internationalization/Text.h"
#include "MediaCaptureSupport.h"
#include "Styling/ISlateStyle.h"
#include "Templates/SharedPointer.h"
#include "UObject/NameTypes.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/SMediaImage.h"

#define UE_API MEDIAPLAYEREDITOR_API

class FMenuBuilder;
class FUICommandList;
class SEditableTextBox;
class SMediaPlayerEditorViewport;
class UMediaPlayer;
class UMediaSoundComponent;
class UMediaTexture;

enum class EMediaEvent;
enum class EMediaPlayerTrack : uint8;


/**
 * Implements the contents of the viewer tab in the UMediaPlayer asset editor.
 */
class SMediaPlayerEditorViewer
	: public SCompoundWidget
{
public:

	SLATE_BEGIN_ARGS(SMediaPlayerEditorViewer)
		: _bShowUrl(true)
	{ }

		/** If the url should be shown. */
		SLATE_ARGUMENT(bool, bShowUrl)
		/** Command list for executing keybinds and mouse clicks. */
		SLATE_ARGUMENT(TSharedPtr<FUICommandList>, Commands)

	SLATE_END_ARGS()

public:

	/** Default constructor. */
	UE_API SMediaPlayerEditorViewer();

	/** Destructor. */
	UE_API ~SMediaPlayerEditorViewer();

public:

	/**
	 * Construct this widget
	 *
	 * @param InArgs The declaration data for this widget.
	 * @param InMediaPlayer The UMediaPlayer asset to show the details for.
	 * @param InMediaTexture The UMediaTexture asset to output video to. If nullptr then use our own.
	 * @param InStyle The style set to use.
	 * @param bInIsSoundEnabled If true then produce sound.
	 */
	UE_API void Construct(const FArguments& InArgs, UMediaPlayer& InMediaPlayer,
		UMediaTexture* InMediaTexture, const TSharedRef<ISlateStyle>& InStyle,
		bool bInIsSoundEnabled);

	UE_API UMediaPlayer* GetMediaPlayer() const;

	/**
	 * Enables/disables using the mouse to control the viewport.
	 */
	UE_API void EnableMouseControl(bool bIsEnabled);

	/**
	 * Returns the active mask.
	 */
	UE_API MediaPlayerEditor::MediaImage::ETextureChannelMask GetChannelMask() const;

	/**
	 * Returns true if all the given channel mask is enabled.
	 */
	bool IsChannelMasked(MediaPlayerEditor::MediaImage::ETextureChannelMask InChannel) const;

	/**
	 * Sets a channel-based mask for the image.
	 *
	 * If only a single channel is displayed, it will show in greyscale.
	 *
	 * @param InMask to display.
	 */
	UE_API void SetChannelMask(MediaPlayerEditor::MediaImage::ETextureChannelMask InMask);

	/**
	 * Toggles the given mask in the channel mask.
	 */
	void ToggleChannelMask(MediaPlayerEditor::MediaImage::ETextureChannelMask InChannelToToggle);

public:

	//~ SWidget interface

	UE_API virtual void OnDragEnter(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent) override;
	UE_API virtual void OnDragLeave(const FDragDropEvent& DragDropEvent) override;
	UE_API virtual FReply OnDragOver(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent) override;
	UE_API virtual FReply OnDrop(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent) override;

protected:

	/**
	 * Populate a menu from the given capture device information.
	 *
	 * @param DeviceInfos The capture device information.
	 * @param MenuBuilder The builder for the menu.
	 */
	UE_API void MakeCaptureDeviceMenu(TArray<FMediaCaptureDeviceInfo>& DeviceInfos, FMenuBuilder& MenuBuilder);

	/** Open the specified media URL. */
	UE_API void OpenUrl(const FText& TextUrl);

	/** Set the name of the desired native media player. */
	UE_API void SetDesiredPlayerName(FName PlayerName);

private:

	/** Callback for creating the audio capture devices sub-menu. */
	UE_API void HandleAudioCaptureDevicesMenuNewMenu(FMenuBuilder& MenuBuilder);

	/** Callback for creating the player sub-menu. */
	UE_API void HandleDecoderMenuNewMenu(FMenuBuilder& MenuBuilder);

	/** Callback for getting the text of the FPS text block. */
	UE_API FText HandleFpsTextBlockText() const;

	/** Callback for creating a track format sub-menu. */
	UE_API void HandleFormatMenuNewMenu(FMenuBuilder& MenuBuilder, EMediaPlayerTrack TrackType);

	/** Callback for media player events. */
	UE_API void HandleMediaPlayerMediaEvent(EMediaEvent Event);

	/** Callback for creating the Scale sub-menu. */
	UE_API void HandleScaleMenuNewMenu(FMenuBuilder& MenuBuilder);

	/** Callback for getting the text of the timer text block. */
	UE_API FText HandleTimerTextBlockText() const;

	/** Callback for getting the tool tip of the timer text block. */
	UE_API FText HandleTimerTextBlockToolTipText() const;

	/** Callback for creating the Background sub-menu. */
	UE_API void HandleBackgroundMenuNewMenu(FMenuBuilder& MenuBuilder);

	/** Callback for creating the timer units sub-menu. */
	UE_API void HandleTimerUnitMenuNewMenu(FMenuBuilder& MenuBuilder);
	
	/** Callback for creating a track sub-menu. */
	UE_API void HandleTrackMenuNewMenu(FMenuBuilder& MenuBuilder, EMediaPlayerTrack TrackType);

	/** Callback for handling key down events in the URL text box. */
	UE_API FReply HandleUrlBoxKeyDown(const FGeometry&, const FKeyEvent& KeyEvent);

	/** Callback for creating the video capture devices sub-menu. */
	UE_API void HandleVideoCaptureDevicesMenuNewMenu(FMenuBuilder& MenuBuilder);

	/** Creates controls for the media */
	UE_API TSharedRef<SWidget> CreatePlayerControls();

private:

	/** Whether something is currently being dragged over the widget. */
	bool DragOver;

	/** Whether the dragged object is a media file that can be played. */
	bool DragValid;

	/** The text that was last typed into the URL box. */
	FText LastUrl;

	/** Pointer to the media player that is being viewed. */
	UMediaPlayer* MediaPlayer;

	/** The style set to use for this widget. */
	TSharedPtr<ISlateStyle> Style;

	/** Media URL text box. */
	TSharedPtr<SEditableTextBox> UrlTextBox;

	/** The viewport that shows the media. */
	TSharedPtr<SMediaPlayerEditorViewport> PlayerViewport;

	/** The controls that can be executed by this viewer. */
	TWeakPtr<FUICommandList> CommandListWeak;
};

#undef UE_API
