// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Styling/ISlateStyle.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"

#define UE_API MEDIAPLAYEREDITOR_API

class SVerticalBox;
class UMediaPlaylist;

/**
 * Implements the tracks view of the MediaPlaylist.
 */
class SMediaPlaylistEditorTracks
	: public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SMediaPlaylistEditorTracks)
		: _ShowAddEntryButton(false)
		{}

		SLATE_ARGUMENT(bool, ShowAddEntryButton)
	SLATE_END_ARGS()

public:
	
	/**
	 * Construct this widget
	 *
	 * @param InArgs The declaration data for this widget.
	 * @param InMediaPlaylist The MediaPlaylist asset to show the details for.
	 * @param InStyleSet The style set to use.
	 */
	UE_API void Construct(const FArguments& InArgs, UMediaPlaylist* InMediaPlaylist, const TSharedRef<ISlateStyle>& InStyle);
	UE_API virtual ~SMediaPlaylistEditorTracks() override;

protected:

	/** Pointer to the MediaPlaylist asset that is being viewed. */
	TWeakObjectPtr<UMediaPlaylist> MediaPlaylistPtr;
	/** Pointer to the container for the media sources. */
	TSharedPtr<SVerticalBox> SourcesContainer;

	/**
	 * Flag to enable/disable a button which can be used to add items to the currently selected Playlist.
	 * Set by "ShowAddEntryButton" slate argument. If no Playlist is selected, button will be hidden anyway.
	 */
	bool bShowAddEntryButton = false;

	/**
	 * This should be called when the playlist changes so we can update the UI.
	 */
	UE_API void RefreshPlaylist();

	/**
	 * Adds a blank entry to the playlist.
	 */
	UE_API void AddToPlaylist();

	/**
	 * Gets the object path for the media source object.
	 * 
	 * @param	Index		Index in the playlist of the media source object.
	 */
	UE_API FString GetMediaSourcePath(int32 Index) const;

	/**
	 * Called when the media source widget changes.
	 *
	 * @param	AssetData	The media source.
	 * @param	Index		Index in the playlist of the media source object.
	 */
	UE_API void OnMediaSourceChanged(const FAssetData& AssetData, int32 Index);

	/** Engine callback to detect when the media is changed. */
	UE_API void OnObjectTransacted(UObject* Object, const FTransactionObjectEvent& TransactionObjectEvent);

	/** Returns the current visibility for the button used to add items to the currently selected Playlist, if any */
	UE_API EVisibility GetAddEntryButtonVisibility() const;

	/** Returns the current visibility of the header area (before or in place of the current Playlist items) */
	UE_API EVisibility GetSourcesContainerHeaderTextVisibility() const;

	/** Returns the text to be displayed in the header area (before or in place of the current Playlist items) */
	UE_API FText GetSourcesContainerHeaderText() const;
};

#undef UE_API
